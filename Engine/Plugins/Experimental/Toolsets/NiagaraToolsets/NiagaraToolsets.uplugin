{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Niagara Toolsets",
	"Description": "A collection of tool calls allowing an AI assistant the ability to interact with Niagara.",
	"Category": "FX",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EditorOnly": true,
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "NiagaraToolsets",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "Niagara",
			"Enabled": true
		},
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		}
	]
}