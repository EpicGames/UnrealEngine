// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace UnrealBuildTool.Rules
{
	public class NiagaraToolsets : ModuleRules
	{
		public NiagaraToolsets(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		

			PrivateDependencyModuleNames.AddRange(
				new[]
				{
					"NiagaraEditor",
					"Core",
					"RHI",
					"RenderCore",
					"CoreUObject",
					"Engine",
					"Niagara",
					"NiagaraCore",
					"ToolsetRegistry",
					"NiagaraBlueprintNodes",
					"Kismet",
					"BlueprintGraph",
					"Json",
					"JsonUtilities",
					"JsonUtilitiesEditor",
					"DeveloperSettings",
				}
			);

			if (Target.bBuildEditor == true)
			{
				
				PrivateDependencyModuleNames.AddRange(
					new[]
					{
						"UnrealEd",
						"StructUtilsEditor",
					}
				);
			}
		}
	}
}
