// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraToolset.h"

#include "Kismet/KismetSystemLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraToolset)

#define LOCTEXT_NAMESPACE "UNiagaraToolset"

void UNiagaraToolset::Error(FText Error)
{
	UKismetSystemLibrary::RaiseScriptError(Error.ToString());
}

void UNiagaraToolset::Error(FString Error)
{
	UKismetSystemLibrary::RaiseScriptError(Error);
}

void UNiagaraToolset::Error(TArrayView<FText> Errors)
{
	for (FText& Err : Errors)
	{
		UKismetSystemLibrary::RaiseScriptError(Err.ToString());
	}
}

bool UNiagaraToolset::ValidateSystem(UNiagaraSystem* System)
{
	if (System == nullptr)
	{
		Error(LOCTEXT("NullSystem", "System was null"));
		return false;
	}
	return true;
}

bool UNiagaraToolset::ValidateComponent(UNiagaraComponent* Component)
{
	if (Component == nullptr)
	{
		Error(LOCTEXT("NullComponent", "Component was null"));
		return false;
	}
	return true;
}

#undef LOCTEXT_NAMESPACE