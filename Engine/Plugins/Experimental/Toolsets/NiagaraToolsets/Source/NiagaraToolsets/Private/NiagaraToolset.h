// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolsetRegistry/ToolsetDefinition.h"
#include "NiagaraToolsetsCommon.h"
#include "NiagaraToolset.generated.h"

class UNiagaraSystem;
class UNiagaraComponent;

/**
 * Base class for all Niagara Toolsets.
 *
 * Provides common validation and error reporting functionality for Niagara-related operations.
 * Derived toolsets handle specific aspects like System editing, Component interaction, and Blueprint integration.
 */
UCLASS(Blueprintable)
class UNiagaraToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:

	/// Validates that a NiagaraSystem pointer is valid, raising a script error if not
	static bool ValidateSystem(UNiagaraSystem* System);

	/// Validates that a NiagaraComponent pointer is valid, raising a script error if not
	static bool ValidateComponent(UNiagaraComponent* Component);

	/// Reports a localized error message to the user via script error system
	static void Error(FText Error);

	/// Reports an error message string to the user via script error system
	static void Error(FString Error);

	/// Reports multiple error messages to the user via script error system
	static void Error(TArrayView<FText> Errors);

	/// Reports a formatted error message with printf-style formatting
	template<typename... ArgTypes>
	static inline void Error(const TCHAR* Fmt, ArgTypes&&... InArgs)
	{
		FStringFormatOrderedArguments Args = { FStringFormatArg(Forward<ArgTypes>(InArgs))... };
		Error(FString::Format(Fmt, Args));
	}
};

