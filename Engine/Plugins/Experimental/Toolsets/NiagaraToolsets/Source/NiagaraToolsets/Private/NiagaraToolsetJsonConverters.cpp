// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraToolsetJsonConverters.h"

#include "NiagaraTypes.h"
#include "Kismet/KismetSystemLibrary.h"

#include "JsonObjectConverter.h"
#include "JsonSchema/JsonSchemaGenerator.h"

#include "NiagaraExternalSystemEditorUtilities.h"

#include "StructUtilsMetadata.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraToolsetJsonConverters)

#define LOCTEXT_NAMESPACE "NiagaraToolsetJsonConverters"


FString FToolsetNiagaraTypeDefinitionConverter::GetName() const
{
	static FString Name(TEXT("ToolsetNiagaraTypeDefinition"));
	return Name;
}

bool FToolsetNiagaraTypeDefinitionConverter::CanConvertProperty(TNotNull<const FProperty*> Property)
{
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		if (const UScriptStruct* ScriptStruct = StructProperty->Struct)
		{
			return ScriptStruct == FNiagaraTypeDefinition::StaticStruct();
		}
	}
	return false;
}

TSharedPtr<FJsonObject> FToolsetNiagaraTypeDefinitionConverter::PropertyToJsonSchema(TNotNull<const FProperty*> Property)
{
	return ToolsetStructToJsonSchema(FToolsetNiagaraTypeDefinition::StaticStruct());
}

TSharedPtr<FJsonValue> FToolsetNiagaraTypeDefinitionConverter::PropertyToDefault(TNotNull<const FProperty*> Property, const FString& DefaultString)
{
	//TODO:
	return MakeShared<FJsonValueNull>();
}

TSharedPtr<FJsonValue> FToolsetNiagaraTypeDefinitionConverter::PropertyToJsonData(TNotNull<FProperty*> Property, const void* Value)
{
	const FNiagaraTypeDefinition* InData = (const FNiagaraTypeDefinition*)Value;
	if (InData == nullptr)
	{
		return nullptr;
	}

	FToolsetNiagaraTypeDefinition ToolsetTypeDef;
	ToolsetTypeDef.ClassStructOrEnum = InData->ClassStructOrEnum;

	if (TSharedPtr<FJsonObject> JsonObject = ToolsetStructToJsonData(
		FToolsetNiagaraTypeDefinition::StaticStruct(), &ToolsetTypeDef))
	{
		return MakeShared<FJsonValueObject>(JsonObject);
	}

	return MakeShared<FJsonValueNull>();
}

bool FToolsetNiagaraTypeDefinitionConverter::JsonDataToProperty(const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property, void* OutValue, UObject* Outer)
{
	TSharedPtr<FJsonObject> JsonObject = JsonValue.IsValid() ? JsonValue->AsObject() : nullptr;
	if(!JsonObject)
	{
		return false;
	}

	FNiagaraTypeDefinition* OutData = (FNiagaraTypeDefinition*)OutValue;
	if (OutData == nullptr)
	{
		return false;
	}

	FToolsetNiagaraTypeDefinition ToolsetType;
	if (ToolsetJsonDataToStruct(
		JsonObject.ToSharedRef(),
		FToolsetNiagaraTypeDefinition::StaticStruct(),
		&ToolsetType))
	{
		if(UScriptStruct* ScriptStruct = Cast<UScriptStruct>(ToolsetType.ClassStructOrEnum))
		{
			*OutData = FNiagaraTypeDefinition(ScriptStruct, FNiagaraTypeDefinition::EAllowUnfriendlyStruct::Allow);
		}
		else if(ToolsetType.ClassStructOrEnum && (ToolsetType.ClassStructOrEnum->IsA<UClass>() || ToolsetType.ClassStructOrEnum->IsA<UEnum>()))
		{
			*OutData = FNiagaraTypeDefinition(ToolsetType.ClassStructOrEnum);
		}
		else
		{
			*OutData = FNiagaraTypeDefinition();
			return false;
		}
		return true;
	}

	UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to convert JSON to FNiagaraTypeDefinition for Property %s."), *Property->GetName()));
	return false;
}

//////////////////////////////////////////////////////////////////////////
// FNiagaraExt_InstancedValue Converter
// Handles JSON conversion for instanced struct values with auto-discovery of valid types
//////////////////////////////////////////////////////////////////////////

FString FToolsetNiagaraInstancedValueConverter::GetName() const
{
	static FString Name(TEXT("NiagaraInstancedValueConverter"));
	return Name;
}

bool FToolsetNiagaraInstancedValueConverter::CanConvertProperty(TNotNull<const FProperty*> Property)
{
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		if (const UScriptStruct* ScriptStruct = StructProperty->Struct)
		{
			return ScriptStruct->IsChildOf(FNiagaraExt_InstancedValue::StaticStruct());
		}
	}
	return false;
}

bool FToolsetNiagaraInstancedValueConverter::DerivedFrom(const UScriptStruct* Struct, const UScriptStruct* Base) const
{
	if (!Struct || !Base)
	{
		return false;
	}

	const UScriptStruct* CurrentStruct = Struct;
	while (CurrentStruct)
	{
		if (CurrentStruct == Base)
		{
			return true;
		}
		CurrentStruct = Cast<UScriptStruct>(CurrentStruct->GetSuperStruct());
	}
	return false;
}

FString FToolsetNiagaraInstancedValueConverter::GetStructDescription(const UScriptStruct* Struct) const
{
	if (!Struct)
	{
		return TEXT("");
	}

	FString Description = Struct->GetMetaData(TEXT("ToolTip"));
	if (Description.IsEmpty())
	{
		Description = FString::Printf(TEXT("Type: %s"), *Struct->GetName());
	}
	return Description;
}

void FToolsetNiagaraInstancedValueConverter::GetValidStructTypes(
	const FStructProperty* StructProperty, TArray<const UScriptStruct*>& OutStructs)
{
	if (!StructProperty)
	{
		return;
	}

	const UScriptStruct* ScriptStruct = StructProperty->Struct;

	FString BaseStructsMeta = StructProperty->GetMetaData(UE::StructUtils::Metadata::BaseStructName);
	if (BaseStructsMeta.IsEmpty())
	{
		BaseStructsMeta = ScriptStruct->GetMetaData(UE::StructUtils::Metadata::BaseStructName);
	}

	if (BaseStructsMeta.IsEmpty())
	{
		return;
	}

	TArray<const UScriptStruct*>* CachedTypes = DerivedTypesCache.Find(BaseStructsMeta);
	if (CachedTypes)
	{
		OutStructs.Append(*CachedTypes);
		return;
	}

	// Parse base struct paths (can be comma-separated list)
	TArray<FString> BaseStructPaths;
	BaseStructsMeta.ParseIntoArray(BaseStructPaths, TEXT(","), true);

	TArray<UScriptStruct*> BaseStructs;
	for (FString& BaseStructPath : BaseStructPaths)
	{
		BaseStructPath.TrimStartAndEndInline();
		UScriptStruct* BaseStruct = FindObject<UScriptStruct>(nullptr, *BaseStructPath);
		if (BaseStruct == nullptr)
		{
			BaseStruct = LoadObject<UScriptStruct>(nullptr, *BaseStructPath);
		}

		// Only include base structs marked with EDAIncludeStructsInSchema
		if (BaseStruct && BaseStruct->HasMetaData(TEXT("EDAIncludeStructsInSchema")))
		{
			BaseStructs.Add(BaseStruct);
		}
	}

	if (BaseStructs.Num() == 0)
	{
		return;
	}

	// Find all structs that derive from any of the base structs
	TArray<const UScriptStruct*> DiscoveredTypes;

	for (TObjectIterator<UScriptStruct> It; It; ++It)
	{
		const UScriptStruct* PossibleChildStruct = *It;

		// Skip structs marked with ExcludeBaseStruct (empty base structs)
		if (PossibleChildStruct->HasMetaData(TEXT("ExcludeBaseStruct")))
		{
			continue;
		}

		for (const UScriptStruct* Base : BaseStructs)
		{
			if (DerivedFrom(PossibleChildStruct, Base))
			{
				DiscoveredTypes.AddUnique(PossibleChildStruct);
				break;
			}
		}
	}

	// Cache the discovered types
	DerivedTypesCache.Add(BaseStructsMeta, DiscoveredTypes);

	// Add to output
	OutStructs.Append(DiscoveredTypes);
}

void FToolsetNiagaraInstancedValueConverter::GetAdditionalTypes(
	const FStructProperty* StructProperty, TArray<const UScriptStruct*>& OutStructs)
{
	if (!StructProperty)
	{
		return;
	}

	const UScriptStruct* ScriptStruct = StructProperty->Struct;

	FString AdditionalTypesMeta = StructProperty->GetMetaData(TEXT("AdditionalTypes"));
	if (AdditionalTypesMeta.IsEmpty() && ScriptStruct)
	{
		AdditionalTypesMeta = ScriptStruct->GetMetaData(TEXT("AdditionalTypes"));
	}

	if (AdditionalTypesMeta.IsEmpty())
	{
		return;
	}

	// Parse additional type paths (comma-separated)
	TArray<FString> AdditionalTypePaths;
	AdditionalTypesMeta.ParseIntoArray(AdditionalTypePaths, TEXT(","), true);

	for (FString& TypePath : AdditionalTypePaths)
	{
		TypePath.TrimStartAndEndInline();

		const UScriptStruct* AdditionalStruct = FindObject<UScriptStruct>(nullptr, *TypePath);
		if (AdditionalStruct == nullptr)
		{
			AdditionalStruct = LoadObject<UScriptStruct>(nullptr, *TypePath);
		}

		if (AdditionalStruct)
		{
			OutStructs.AddUnique(AdditionalStruct);
		}
	}
}

TSharedPtr<FJsonObject> FToolsetNiagaraInstancedValueConverter::BuildOneOfSchema(
	const TArray<const UScriptStruct*>& AllowedTypes, const FString& Description, int32 SpecializedTypeCount)
{
	TSharedPtr<FJsonObject> Schema = MakeShared<FJsonObject>();
	TArray<TSharedPtr<FJsonValue>> OneOfArray;

	for (const UScriptStruct* StructType : AllowedTypes)
	{
		TSharedPtr<FJsonObject> TypeSchema;

		if (ExpansionStack.Contains(StructType))
		{
			// Re-entering a struct already being expanded upstream — emit a terminal inline
			// stub instead of recursing. Keeps the schema free of $ref while still conveying
			// the recursion to consumers.
			TypeSchema = MakeRecursiveStubSchema(StructType);
		}
		else
		{
			ExpansionStack.Push(StructType);
			TypeSchema = ToolsetStructToJsonSchema(StructType);
			ExpansionStack.Pop();
		}

		if (TypeSchema.IsValid())
		{
			TypeSchema->SetStringField(TEXT("title"), StructType->GetPathName());

			FString StructDescription = GetStructDescription(StructType);
			if (!StructDescription.IsEmpty() && !TypeSchema->HasField(TEXT("description")))
			{
				TypeSchema->SetStringField(TEXT("description"), StructDescription);
			}

			OneOfArray.Add(MakeShared<FJsonValueObject>(TypeSchema));
		}
	}

	Schema->SetArrayField(TEXT("oneOf"), OneOfArray);

	FString EnhancedDescription = Description;
	if (SpecializedTypeCount > 0 && SpecializedTypeCount < AllowedTypes.Num())
	{
		EnhancedDescription += FString::Printf(
			TEXT(" The first %d type(s) are specialized types for this context. Additional types may also be valid depending on usage."),
			SpecializedTypeCount
		);
	}
	Schema->SetStringField(TEXT("description"), EnhancedDescription);

	return Schema;
}

TSharedPtr<FJsonObject> FToolsetNiagaraInstancedValueConverter::MakeRecursiveStubSchema(const UScriptStruct* Struct) const
{
	TSharedPtr<FJsonObject> Stub = MakeShared<FJsonObject>();
	Stub->SetStringField(TEXT("type"), TEXT("object"));

	if (!Struct)
	{
		Stub->SetStringField(TEXT("description"), TEXT("Recursive reference — same shape repeats to arbitrary depth."));
		return Stub;
	}

	// Build a brief field inventory from reflection so the stub carries enough information
	// for consumers to recognise the recursive shape without cross-referencing the enclosing oneOf.
	TArray<FString> FieldNames;
	for (TFieldIterator<FProperty> It(Struct); It; ++It)
	{
		FieldNames.Add(It->GetName());
	}

	const FString StructName = Struct->GetName();
	FString Description;
	if (FieldNames.Num() > 0)
	{
		Description = FString::Printf(
			TEXT("Recursive reference to %s — same shape repeats to arbitrary depth. Fields: %s."),
			*StructName, *FString::Join(FieldNames, TEXT(", ")));
	}
	else
	{
		Description = FString::Printf(
			TEXT("Recursive reference to %s — same shape repeats to arbitrary depth."),
			*StructName);
	}

	Stub->SetStringField(TEXT("description"), Description);
	return Stub;
}

TSharedPtr<FJsonObject> FToolsetNiagaraInstancedValueConverter::PropertyToJsonSchema(TNotNull<const FProperty*> Property)
{
	const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
	check(StructProperty);
	const UScriptStruct* ScriptStruct = StructProperty->Struct;
	check(ScriptStruct && ScriptStruct->IsChildOf(FNiagaraExt_InstancedValue::StaticStruct()));

	TSharedPtr<FJsonObject> Schema = ToolsetStructToJsonSchema(FNiagaraToolsetInstancedValue::StaticStruct());

	const TSharedPtr<FJsonObject>* PropertiesObj = nullptr;
	if (Schema->TryGetObjectField(TEXT("properties"), PropertiesObj))
	{
		TArray<const UScriptStruct*> AllowedTypes;

		// Priority 1: Auto-discover specialized types from BaseStruct metadata
		GetValidStructTypes(StructProperty, AllowedTypes);
		int32 SpecializedTypeCount = AllowedTypes.Num();

		// Priority 2: Add additional types from AdditionalTypes metadata
		GetAdditionalTypes(StructProperty, AllowedTypes);

		if (AllowedTypes.Num() == 0)
		{
			TSharedPtr<FJsonObject> FallbackSchema = MakeShared<FJsonObject>();
			FallbackSchema->SetStringField(TEXT("type"), TEXT("object"));
			FallbackSchema->SetStringField(TEXT("description"), TEXT("Instance of a struct. No specific types defined - check BaseStruct or AdditionalTypes metadata."));
			(*PropertiesObj)->SetObjectField(TEXT("value"), FallbackSchema);
		}
		else
		{
			FString Description = FString::Printf(
				TEXT("Value can be one of %d types defined by Struct."),
				AllowedTypes.Num());

			TSharedPtr<FJsonObject> ValueSchema = BuildOneOfSchema(AllowedTypes, Description, SpecializedTypeCount);
			(*PropertiesObj)->SetObjectField(TEXT("value"), ValueSchema);
		}
	}

	return Schema;
}

TSharedPtr<FJsonValue> FToolsetNiagaraInstancedValueConverter::PropertyToDefault(TNotNull<const FProperty*> Property, const FString& DefaultString)
{
	// Return empty default for now
	TSharedPtr<FJsonObject> DefaultObject = MakeShared<FJsonObject>();
	return MakeShared<FJsonValueObject>(DefaultObject);
}

TSharedPtr<FJsonValue> FToolsetNiagaraInstancedValueConverter::PropertyToJsonData(TNotNull<FProperty*> Property, const void* Value)
{

	const FNiagaraExt_InstancedValue* InData = static_cast<const FNiagaraExt_InstancedValue*>(Value);
	if (InData == nullptr || !InData->IsValid())
	{
		return nullptr;
	}

	FNiagaraToolsetInstancedValue ToolsetInst;
	ToolsetInst.Struct = InData->GetScriptStruct();

	if (TSharedPtr<FJsonObject> OutJson = ToolsetStructToJsonData(
		FNiagaraToolsetInstancedValue::StaticStruct(), &ToolsetInst))
	{
		if (TSharedPtr<FJsonObject> ValueObj = ToolsetStructToJsonData(
			InData->GetScriptStruct(), InData->GetMemory()))
		{
			OutJson->SetObjectField(TEXT("value"), ValueObj);
			return MakeShared<FJsonValueObject>(OutJson);
		}
	}

	UKismetSystemLibrary::RaiseScriptError(FString::Printf(
		TEXT("Failed to convert FNiagaraExt_InstancedValue to JSON for Property %s."), *Property->GetName()));
	return MakeShared<FJsonValueNull>();
}

bool FToolsetNiagaraInstancedValueConverter::JsonDataToProperty(const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property, void* OutValue, UObject* Outer)
{

	TSharedPtr<FJsonObject> JsonObject = JsonValue.IsValid() ? JsonValue->AsObject() : nullptr;
	if(!JsonObject)
	{
		return false;
	}

	FNiagaraExt_InstancedValue* OutData = static_cast<FNiagaraExt_InstancedValue*>(OutValue);
	if (OutData == nullptr)
	{
		return false;
	}

	FNiagaraToolsetInstancedValue ToolsetInstStruct;
	if (ToolsetJsonDataToStruct(JsonObject.ToSharedRef(), FNiagaraToolsetInstancedValue::StaticStruct(), &ToolsetInstStruct))
	{
		if (ToolsetInstStruct.Struct == nullptr)
		{
			return false;
		}

		const TSharedPtr<FJsonObject>* ValueObject = nullptr;
		if (JsonObject->TryGetObjectField(TEXT("value"), ValueObject))
		{
			OutData->InitializeAs(ToolsetInstStruct.Struct);
			if (ToolsetJsonDataToStruct(ValueObject->ToSharedRef(), OutData->GetScriptStruct(), OutData->GetMutableMemory()))
			{
				return true;
			}
		}
	}

	UKismetSystemLibrary::RaiseScriptError(FString::Printf(
		TEXT("Failed to convert JSON to FNiagaraExt_InstancedValue for Property %s."), *Property->GetName()));
	return false;
}

#undef LOCTEXT_NAMESPACE
