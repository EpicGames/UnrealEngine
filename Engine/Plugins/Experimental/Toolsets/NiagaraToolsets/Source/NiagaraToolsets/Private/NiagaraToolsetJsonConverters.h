// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolsetRegistry/ToolsetJsonConverter.h"

#include "NiagaraToolsetJsonConverters.generated.h"

/**
 * Wrapper struct for FNiagaraTypeDefinition that exposes only the UStruct pointer for JSON conversion.
 *
 * FNiagaraTypeDefinition is a complex struct with internal state. This wrapper simplifies
 * JSON conversion by only exposing the Struct field, which is sufficient for reconstruction.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FToolsetNiagaraTypeDefinition
{
	GENERATED_BODY()

	/** Reference to the UScriptStruct, UEnum, or UClass that defines this Niagara type. */
	UPROPERTY(EditAnywhere, Category=Variant)
	TObjectPtr<UObject> ClassStructOrEnum;
};

/**
 * Custom JSON converter for FNiagaraTypeDefinition properties.
 *
 * Converts between FNiagaraTypeDefinition and JSON by using the simplified
 * FToolsetNiagaraTypeDefinition wrapper struct.
 */
class FToolsetNiagaraTypeDefinitionConverter : public UE::ToolsetRegistry::FToolsetJsonConverter
{
public:
	virtual FString GetName() const override;
	virtual bool CanConvertProperty(TNotNull<const FProperty*> Property) override;
	virtual TSharedPtr<FJsonObject> PropertyToJsonSchema(TNotNull<const FProperty*> Property) override;
	virtual TSharedPtr<FJsonValue> PropertyToDefault(TNotNull<const FProperty*> Property, const FString& DefaultString) override;
	virtual TSharedPtr<FJsonValue> PropertyToJsonData(TNotNull<FProperty*> Property, const void* Value) override;
	virtual bool JsonDataToProperty(const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property, void* OutValue, UObject* Outer) override;
};

//////////////////////////////////////////////////////////////////////////
// FNiagaraExtInstancedValue JSON Converters

/**
 * Wrapper struct for FInstancedStruct that simplifies JSON conversion.
 *
 * Represents an instanced value that can be one of multiple allowed struct types.
 * The allowed types are determined by BaseStruct metadata (auto-discovered derived types)
 * and optional AdditionalTypes metadata (explicitly specified types).
 *
 * When working with this type, check the JSON schema's oneOf array to see all valid types.
 * Specialized types (derived from BaseStruct) are listed first, followed by additional types.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraToolsetInstancedValue
{
	GENERATED_BODY()

	/**
	 * The ScriptStruct type that defines the data structure for this instanced value.
	 * Must be one of the allowed types defined in the schema's oneOf array.
	 *
	 * Specialized types (if any) should be preferred when applicable.
	 * Refer to the schema descriptions for guidance on when to use each type.
	 */
	UPROPERTY(EditAnywhere, Category=Variant, meta=(
		ToolTip="The struct type for this instanced value. Must match one of the allowed types in the schema."
	))
	TObjectPtr<const UScriptStruct> Struct;
};

/**
 * Custom JSON converter for FNiagaraExt_InstancedValue properties.
 *
 * Automatically discovers valid types using BaseStruct metadata (derived types)
 * and AdditionalTypes metadata (explicitly included types), generating comprehensive
 * JSON schemas with oneOf arrays and helpful descriptions to guide AI usage.
 *
 * Type discovery is cached for performance.
 */
class FToolsetNiagaraInstancedValueConverter : public UE::ToolsetRegistry::FToolsetJsonConverter
{
public:
	virtual FString GetName() const override;
	virtual bool CanConvertProperty(TNotNull<const FProperty*> Property) override;
	virtual TSharedPtr<FJsonObject> PropertyToJsonSchema(TNotNull<const FProperty*> Property) override;
	virtual TSharedPtr<FJsonValue> PropertyToDefault(TNotNull<const FProperty*> Property, const FString& DefaultString) override;
	virtual TSharedPtr<FJsonValue> PropertyToJsonData(TNotNull<FProperty*> Property, const void* Value) override;
	virtual bool JsonDataToProperty(const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property, void* OutValue, UObject* Outer) override;

private:
	/// Discovers all valid struct types using BaseStruct metadata (auto-discovers derived types)
	void GetValidStructTypes(const FStructProperty* StructProperty, TArray<const UScriptStruct*>& OutStructs);

	/// Checks if a struct derives from a base struct
	bool DerivedFrom(const UScriptStruct* Struct, const UScriptStruct* Base) const;

	/// Gets additional types to include based on AdditionalTypes metadata
	void GetAdditionalTypes(const FStructProperty* StructProperty, TArray<const UScriptStruct*>& OutStructs);

	/// Extracts description from struct metadata to provide context in JSON schema
	FString GetStructDescription(const UScriptStruct* Struct) const;

	/// Builds a oneOf JSON schema with descriptions for each allowed type
	TSharedPtr<FJsonObject> BuildOneOfSchema(const TArray<const UScriptStruct*>& AllowedTypes, const FString& Description, int32 SpecializedTypeCount);

	/// Builds a terminal inline schema used at recursion boundaries. Avoids $ref so the schema
	/// stays consumable by validators without cross-reference support. The stub carries the
	/// struct's path in `title` and a human-readable field inventory in `description`.
	TSharedPtr<FJsonObject> MakeRecursiveStubSchema(const UScriptStruct* Struct) const;

	/// Cache for discovered derived types to avoid repeated reflection queries
	TMap<FString, TArray<const UScriptStruct*>> DerivedTypesCache;

	/// Stack of struct types currently being expanded into the schema. When a struct
	/// re-enters expansion (self- or mutually-recursive FInstancedStruct wrappers), the
	/// second encounter is replaced with the recursion stub from MakeRecursiveStubSchema.
	TArray<const UScriptStruct*> ExpansionStack;
};