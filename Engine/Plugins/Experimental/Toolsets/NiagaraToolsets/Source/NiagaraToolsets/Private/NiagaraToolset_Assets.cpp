// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraToolset_Assets.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/ARFilter.h"

#include "NiagaraToolsetsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraToolset_Assets)

#define LOCTEXT_NAMESPACE "UNiagaraToolset_Assets"

TArray<FNiagaraToolsetAssetDiscoveryGroup> UNiagaraToolset_Assets::GetAssetDiscoveryInfo()
{
	const UNiagaraToolsetsSettings* Settings = GetDefault<UNiagaraToolsetsSettings>();
	if (Settings)
	{
		return Settings->AssetDiscoveryGroups;
	}

	return TArray<FNiagaraToolsetAssetDiscoveryGroup>();
}

namespace
{
	static const FName TagUsage(TEXT("Usage"));
	static const FName TagVisibility(TEXT("LibraryVisibility"));
	static const FName TagModuleUsageBitmask(TEXT("ModuleUsageBitmask"));
	static const FName TagCategory(TEXT("Category"));
	static const FName TagDescription(TEXT("Description"));
	static const FName TagKeywords(TEXT("Keywords"));
	static const FName TagDeprecated(TEXT("bDeprecated"));
	static const FName TagSuggested(TEXT("bSuggested"));

	/**
	 * Strips any "EnumName::" prefix from an enum tag value string.
	 *
	 * Auto-emitted tags for AssetRegistrySearchable enum properties and the
	 * manually-emitted LibraryVisibility tag have historically used either the
	 * qualified form ("ENiagaraScriptUsage::Module") or the short form ("Module")
	 * depending on UE version and code path. Normalising both incoming filter
	 * values and emitted tag values to the short form makes comparison
	 * format-agnostic.
	 */
	FString ShortForm(const FString& In)
	{
		int32 Pos = INDEX_NONE;
		if (In.FindLastChar(TEXT(':'), Pos))
		{
			return In.RightChop(Pos + 1);
		}
		return In;
	}

	template <typename TEnum>
	FString EnumShortName(TEnum Value)
	{
		const UEnum* EnumObj = StaticEnum<TEnum>();
		return EnumObj ? ShortForm(EnumObj->GetNameStringByValue(static_cast<int64>(Value))) : FString();
	}

	template <typename TEnum>
	int64 ParseEnumTag(const FString& TagValue)
	{
		const UEnum* EnumObj = StaticEnum<TEnum>();
		if (!EnumObj || TagValue.IsEmpty())
		{
			return INDEX_NONE;
		}
		// Try the value as-is, then short form, then short form re-qualified.
		int64 Val = EnumObj->GetValueByNameString(TagValue);
		if (Val != INDEX_NONE)
		{
			return Val;
		}
		const FString Short = ShortForm(TagValue);
		Val = EnumObj->GetValueByNameString(Short);
		if (Val != INDEX_NONE)
		{
			return Val;
		}
		const FString Qualified = FString::Printf(TEXT("%s::%s"), *EnumObj->GetName(), *Short);
		return EnumObj->GetValueByNameString(Qualified);
	}
}

TArray<FAssetData> UNiagaraToolset_Assets::FindNiagaraScripts(
	FString FolderPath,
	FString Name,
	TArray<ENiagaraScriptUsage> Usages,
	TArray<ENiagaraScriptLibraryVisibility> Visibilities,
	int32 ModuleUsageBitmask,
	bool bRecursive,
	bool bIncludeDeprecated)
{
	IAssetRegistry& AR = IAssetRegistry::GetChecked();
	const FTopLevelAssetPath ScriptClass = UNiagaraScript::StaticClass()->GetClassPathName();

	TArray<FAssetData> Found;
	if (!FolderPath.IsEmpty())
	{
		FARFilter Filter;
		Filter.ClassPaths.Add(ScriptClass);
		Filter.PackagePaths.Add(*FolderPath);
		Filter.bRecursivePaths = bRecursive;
		AR.GetAssets(Filter, Found);
	}
	else
	{
		AR.GetAssetsByClass(ScriptClass, Found, /*bSearchSubClasses=*/false);
	}

	TSet<FString> AllowedUsages;
	AllowedUsages.Reserve(Usages.Num());
	for (ENiagaraScriptUsage U : Usages)
	{
		AllowedUsages.Add(EnumShortName(U));
	}

	TSet<FString> AllowedVisibilities;
	if (Visibilities.IsEmpty())
	{
		AllowedVisibilities.Add(EnumShortName(ENiagaraScriptLibraryVisibility::Library));
	}
	else
	{
		AllowedVisibilities.Reserve(Visibilities.Num());
		for (ENiagaraScriptLibraryVisibility V : Visibilities)
		{
			AllowedVisibilities.Add(EnumShortName(V));
		}
	}

	const FString ModuleUsageName = EnumShortName(ENiagaraScriptUsage::Module);
	const FString NameLower = Name.ToLower();

	TArray<FAssetData> Results;
	Results.Reserve(Found.Num());

	for (const FAssetData& Asset : Found)
	{
		if (!NameLower.IsEmpty() && !Asset.AssetName.ToString().ToLower().Contains(NameLower))
		{
			continue;
		}

		FString AssetUsage;
		Asset.GetTagValue(TagUsage, AssetUsage);
		AssetUsage = ShortForm(AssetUsage);

		if (AllowedUsages.Num() > 0 && !AllowedUsages.Contains(AssetUsage))
		{
			continue;
		}

		FString AssetVisibility;
		Asset.GetTagValue(TagVisibility, AssetVisibility);
		AssetVisibility = ShortForm(AssetVisibility);

		if (!AllowedVisibilities.Contains(AssetVisibility))
		{
			continue;
		}

		if (!bIncludeDeprecated)
		{
			int32 AssetDeprecated = 0;
			Asset.GetTagValue(TagDeprecated, AssetDeprecated);
			if (AssetDeprecated != 0)
			{
				continue;
			}
		}

		if (ModuleUsageBitmask != 0)
		{
			if (AssetUsage != ModuleUsageName)
			{
				continue;
			}
			int32 AssetBitmask = 0;
			Asset.GetTagValue(TagModuleUsageBitmask, AssetBitmask);
			if ((AssetBitmask & ModuleUsageBitmask) == 0)
			{
				continue;
			}
		}

		Results.Add(Asset);
	}

	return Results;
}

FNiagaraExt_ScriptDigest UNiagaraToolset_Assets::GetNiagaraScriptDigest(FString ObjectPath)
{
	FNiagaraExt_ScriptDigest Digest;

	if (ObjectPath.IsEmpty())
	{
		Error(LOCTEXT("EmptyObjectPath", "ObjectPath must not be empty."));
		return Digest;
	}

	IAssetRegistry& AR = IAssetRegistry::GetChecked();
	const FAssetData Asset = AR.GetAssetByObjectPath(FSoftObjectPath(ObjectPath));

	if (!Asset.IsValid())
	{
		Error(FText::Format(LOCTEXT("AssetNotFound", "No asset found at path '{0}'."),
			FText::FromString(ObjectPath)));
		return Digest;
	}

	if (Asset.AssetClassPath != UNiagaraScript::StaticClass()->GetClassPathName())
	{
		Error(FText::Format(LOCTEXT("NotAScript", "Asset at path '{0}' is not a UNiagaraScript (got '{1}')."),
			FText::FromString(ObjectPath),
			FText::FromString(Asset.AssetClassPath.ToString())));
		return Digest;
	}

	Digest.ObjectPath = Asset.GetSoftObjectPath().ToString();
	Digest.PackageName = Asset.PackageName.ToString();
	Digest.AssetName = Asset.AssetName.ToString();

	FString UsageStr;
	if (Asset.GetTagValue(TagUsage, UsageStr))
	{
		const int64 Val = ParseEnumTag<ENiagaraScriptUsage>(UsageStr);
		if (Val != INDEX_NONE)
		{
			Digest.Usage = static_cast<ENiagaraScriptUsage>(Val);
		}
	}

	FString VisibilityStr;
	if (Asset.GetTagValue(TagVisibility, VisibilityStr))
	{
		const int64 Val = ParseEnumTag<ENiagaraScriptLibraryVisibility>(VisibilityStr);
		if (Val != INDEX_NONE)
		{
			Digest.LibraryVisibility = static_cast<ENiagaraScriptLibraryVisibility>(Val);
		}
	}

	Asset.GetTagValue(TagModuleUsageBitmask, Digest.ModuleUsageBitmask);
	Asset.GetTagValue(TagCategory, Digest.Category);
	Asset.GetTagValue(TagDescription, Digest.Description);
	Asset.GetTagValue(TagKeywords, Digest.Keywords);

	int32 DeprecatedInt = 0;
	if (Asset.GetTagValue(TagDeprecated, DeprecatedInt))
	{
		Digest.bDeprecated = DeprecatedInt != 0;
	}

	int32 SuggestedInt = 0;
	if (Asset.GetTagValue(TagSuggested, SuggestedInt))
	{
		Digest.bSuggested = SuggestedInt != 0;
	}

	return Digest;
}

#undef LOCTEXT_NAMESPACE
