// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraToolset.h"
#include "NiagaraScript.h"
#include "AssetRegistry/AssetData.h"

#include "NiagaraToolset_Assets.generated.h"

/**
 * Decoded metadata for a Niagara script asset.
 *
 * Flat record of the asset-registry tags emitted by UNiagaraScript, plus the
 * asset's identity. Filled in by GetNiagaraScriptDigest from a script asset's
 * object path. No LoadObject is performed - the digest is built entirely from
 * registry tags. Empty FString fields indicate the corresponding tag is unset
 * (e.g. no Category text on the script).
 */
USTRUCT(BlueprintType)
struct FNiagaraExt_ScriptDigest
{
	GENERATED_BODY()

	/// Full object path: /Path/To/Package.AssetName
	UPROPERTY(EditAnywhere, Category = "Script")
	FString ObjectPath;

	/// Long package name: /Path/To/Package
	UPROPERTY(EditAnywhere, Category = "Script")
	FString PackageName;

	/// Asset name without package
	UPROPERTY(EditAnywhere, Category = "Script")
	FString AssetName;

	/// Script kind: Module, DynamicInput, Function, or one of the stage scripts
	UPROPERTY(EditAnywhere, Category = "Script")
	ENiagaraScriptUsage Usage = ENiagaraScriptUsage::Module;

	/// Library visibility: Unexposed, Library (= "Exposed" in editor UI), or Hidden
	UPROPERTY(EditAnywhere, Category = "Script")
	ENiagaraScriptLibraryVisibility LibraryVisibility = ENiagaraScriptLibraryVisibility::Unexposed;

	/// Bitmask of stack contexts a Module is compatible with. 0 for non-Module scripts.
	UPROPERTY(EditAnywhere, Category = "Script")
	int32 ModuleUsageBitmask = 0;

	/// Library category text (empty if unset)
	UPROPERTY(EditAnywhere, Category = "Script")
	FString Category;

	/// Library description text (empty if unset)
	UPROPERTY(EditAnywhere, Category = "Script")
	FString Description;

	/// Library search keywords (empty if unset)
	UPROPERTY(EditAnywhere, Category = "Script")
	FString Keywords;

	/// True if the script is marked deprecated
	UPROPERTY(EditAnywhere, Category = "Script")
	bool bDeprecated = false;

	/// True if the script is marked suggested
	UPROPERTY(EditAnywhere, Category = "Script")
	bool bSuggested = false;
};

/**
 * Asset discovery group defining content directory paths.
 *
 * Groups asset paths by purpose (templates, modules, materials, etc.)
 * with descriptions to guide asset discovery.
 */
USTRUCT(BlueprintType)
struct FNiagaraToolsetAssetDiscoveryGroup
{
	GENERATED_BODY()

	/// Description of what types of assets are in these paths and when to use them
	UPROPERTY(EditAnywhere, Category="Asset Group")
	FString Description;

	/// Content directory paths containing assets for this group
	UPROPERTY(EditAnywhere, Category="Asset Group")
	TArray<FString> Paths;
};

/**
 * Niagara Toolset for discovery of Niagara script assets.
 *
 * Provides:
 * - GetAssetDiscoveryInfo: project-configured asset discovery groups
 * - FindNiagaraScripts: asset registry search filtered by usage, visibility, name, and module usage bitmask
 * - GetNiagaraScriptDigest: decoded asset-registry tags for a single script
 *
 * All functions read from the asset registry without LoadObject. Pair with
 * NiagaraToolset_System.AddModule to drop discovered modules into a stack.
 *
 * Note: FAssetData JSON serialization does not include the TagsAndValues map (it
 * is not a UPROPERTY), so callers wanting decoded metadata for a result row from
 * FindNiagaraScripts must round-trip through GetNiagaraScriptDigest.
 */
UCLASS(Blueprintable)
class UNiagaraToolset_Assets : public UNiagaraToolset
{
	GENERATED_BODY()

public:

	/**
	 * Returns the project's configured asset discovery groups.
	 * Each group describes a content directory's purpose and paths.
	 *
	 * @return Configured asset discovery groups, or an empty array if none are configured in NiagaraToolsetsSettings.
	 */
	UFUNCTION(meta = (AICallable), Category = "Assets")
	static TArray<FNiagaraToolsetAssetDiscoveryGroup> GetAssetDiscoveryInfo();

	/**
	 * Searches for UNiagaraScript assets matching the given filters.
	 *
	 * Reads filterable metadata from asset registry tags only - no LoadObject required.
	 *
	 * Tags reflect the exposed (published) version of versioned scripts; this
	 * function does not need a version filter and there is no way to discover
	 * non-exposed versions through the asset registry.
	 *
	 * @param FolderPath The folder to search within. Pass an empty string to search the entire project.
	 * @param Name If non-empty, only return assets whose name contains this substring (case-insensitive).
	 * @param Usages If non-empty, only return assets whose Usage matches one of the listed values. Empty array = all usages allowed.
	 * @param Visibilities If non-empty, only return assets whose LibraryVisibility matches one of the listed values. Empty array defaults to {Library} (the editor's "Exposed") - explicit override required to surface Unexposed or Hidden scripts.
	 * @param ModuleUsageBitmask If non-zero, restricts results to Module scripts whose ModuleUsageBitmask shares at least one bit with this argument (any-match). Non-Module scripts are excluded when this is non-zero. Pass 0 to disable the bitmask gate.
	 * @param bRecursive Whether to search subfolders. Ignored when FolderPath is empty (whole-project scan is always exhaustive).
	 * @param bIncludeDeprecated If false (default usage), assets whose exposed version has bDeprecated=true are excluded. Pass true only when the caller explicitly wants to surface deprecated assets (e.g. searching for the canonical replacement of a known deprecated module).
	 * @return Array of AssetData entries for matching scripts. Use GetNiagaraScriptDigest to decode tag metadata for any row.
	 */
	UFUNCTION(meta = (AICallable), Category = "Assets")
	static TArray<FAssetData> FindNiagaraScripts(
		FString FolderPath,
		FString Name,
		TArray<ENiagaraScriptUsage> Usages,
		TArray<ENiagaraScriptLibraryVisibility> Visibilities,
		int32 ModuleUsageBitmask,
		bool bRecursive,
		bool bIncludeDeprecated);

	/**
	 * Returns the decoded asset-registry tag metadata for a Niagara script asset.
	 *
	 * Looks up the asset by object path in the asset registry and reads its tags;
	 * no LoadObject is performed. Returned fields reflect the exposed (published)
	 * version when the script uses FVersionedNiagaraScriptData versioning - the
	 * registry never carries non-exposed-version metadata.
	 *
	 * @param ObjectPath Full object path of the script (e.g. "/Niagara/Modules/Spawn/Initialize Particle.Initialize Particle").
	 * @return Decoded digest. If the path is invalid or not a UNiagaraScript, an error is raised and a default-initialized digest is returned.
	 */
	UFUNCTION(meta = (AICallable), Category = "Assets")
	static FNiagaraExt_ScriptDigest GetNiagaraScriptDigest(FString ObjectPath);
};
