// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraToolset_Blueprint.h"

#include "NiagaraToolsetsCommon.h"
#include "NiagaraToolsetsSettings.h"

#include "NiagaraSystem.h"
#include "NiagaraComponent.h"
#include "NiagaraBlueprintUtil.h"
#include "NiagaraEditorUtilities.h"

#include "Engine/Blueprint.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/SCS_Node.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_Niagara.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "K2Node_CallFunction.h"
#include "K2Node_VariableGet.h"
#include "K2Node_FunctionEntry.h"

#include "AssetToolsModule.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "NiagaraExternalSystemEditorUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraToolset_Blueprint)

#define LOCTEXT_NAMESPACE "UNiagaraToolset_Blueprint"


UBlueprint* UNiagaraToolset_Blueprint::CreateNewBPAsset_Internal(const FString& NewAssetPath, UClass* ParentClass)
{
	if (ParentClass == nullptr)
	{
		return nullptr;
	}

	// Create a blueprint
	FString PackageName;
	FString AssetName = FPackageName::GetLongPackageAssetName(NewAssetPath);

	// If no AssetName was found, generate a unique asset name.
	if (AssetName.Len() == 0)
	{
		PackageName = FPackageName::GetLongPackagePath(NewAssetPath);
		FString BasePath = PackageName + TEXT("/") + LOCTEXT("BlueprintName_Default", "NewBlueprint").ToString();
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(BasePath, TEXT(""), PackageName, AssetName);
	}
	else
	{
		PackageName = NewAssetPath;
	}

	if (FPackageName::IsValidLongPackageName(PackageName))
	{
		//Construct our new wrapper BP.
		if (UPackage* NewAssetPackage = CreatePackage(*PackageName))
		{
			if (UBlueprint* NewBP = FKismetEditorUtilities::CreateBlueprint(ParentClass, NewAssetPackage, *AssetName, BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), TEXT("ConstructNiagaraSystemBPWrapper")))
			{
				FAssetRegistryModule::AssetCreated(NewBP);
				NewAssetPackage->MarkPackageDirty();
				return NewBP;
			}
		}
	}

	return nullptr;
}

UEdGraph* UNiagaraToolset_Blueprint::GetConstructionScriptGraph(UBlueprint* BP)
{
	UEdGraph* Graph = FBlueprintEditorUtils::FindUserConstructionScript(BP);
	if (!Graph)
	{
		Graph = FKismetEditorUtilities::CreateUserConstructionScript(BP);
	}
	return Graph;
}

UActorComponent* UNiagaraToolset_Blueprint::AddComponent_Internal(UBlueprint* BP, TSubclassOf<UActorComponent> ComponentClass, FString ComponentName)
{
	if (BP == nullptr || ComponentClass == nullptr)
	{
		return nullptr;
	}

	if (ComponentName.IsEmpty())
	{
		ComponentName = ComponentClass->GetName();
	}

	if (USimpleConstructionScript* ConstructionScript = BP->SimpleConstructionScript)
	{
		USCS_Node* DefaultRootNode = ConstructionScript->GetDefaultSceneRootNode();
		USCS_Node* CompNode = ConstructionScript->CreateNode(ComponentClass, *ComponentName);
		if (DefaultRootNode)
		{
			DefaultRootNode->AddChildNode(CompNode);
		}
		else
		{
			ConstructionScript->AddNode(CompNode);
		}

		return CompNode->ComponentTemplate;
	}
	return nullptr;

}

bool UNiagaraToolset_Blueprint::AddVariable_Internal(UBlueprint* Blueprint, FName Name, FEdGraphPinType Type, FString Tooltip, FString DefaultValue)
{
	bool bResult = FBlueprintEditorUtils::AddMemberVariable(Blueprint, Name, Type, DefaultValue);
	if (bResult)
	{
		FBlueprintEditorUtils::SetBlueprintVariableMetaData(Blueprint, Name, nullptr, FBlueprintMetadata::MD_Tooltip, Tooltip);
		FBlueprintEditorUtils::SetBlueprintVariableMetaData(Blueprint, Name, nullptr, FBlueprintMetadata::MD_ExposeOnSpawn, TEXT("true"));

		if (uint64* PropFlagPtr = FBlueprintEditorUtils::GetBlueprintVariablePropertyFlags(Blueprint, Name))
		{
			*PropFlagPtr &= ~CPF_DisableEditOnInstance;
			*PropFlagPtr |= CPF_Edit;
			*PropFlagPtr |= CPF_BlueprintVisible;
		}

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}

	return bResult;
}


UBlueprint* UNiagaraToolset_Blueprint::ConstructNiagaraBPWrapperFromComponent(const FString& NewAssetPath, UNiagaraComponent* Component, UClass* ParentClass)
{
	if (ParentClass)
	{
		if (ParentClass->IsChildOf(AActor::StaticClass()) == false)
		{
			Error(TEXT("Parent class is not a child of AActor"));
			return nullptr;
		}
	}
	else
	{
		Error(TEXT("Invalid ParentClassPath"));
		return nullptr;
	}

	UBlueprint* NewBP = nullptr;
	UNiagaraSystem* System = Component->GetAsset();
	if (System)
	{
		NewBP = ConstructBPWrapper_Internal(NewAssetPath, System, Component, ParentClass);
	}
	else
	{
		Error(TEXT("Component has no set Niagara System Asset."));
	}

	return NewBP;
}

UBlueprint* UNiagaraToolset_Blueprint::ConstructNiagaraBPWrapperFromSystem(const FString& NewAssetPath, UNiagaraSystem* System, UClass* ParentClass)
{
	if (ParentClass)
	{
		if (ParentClass->IsChildOf(AActor::StaticClass()) == false)
		{
			Error(TEXT("Parent class is not a child of AActor"));
			return nullptr;
		}
	}
	else
	{
		Error(TEXT("Invalid ParentClassPath"));
		return nullptr;
	}

	UBlueprint* NewBP = ConstructBPWrapper_Internal(NewAssetPath, System, nullptr, ParentClass);
	return NewBP;
}


UBlueprint* UNiagaraToolset_Blueprint::ConstructBPWrapper_Internal(const FString& NewAssetPath, UNiagaraSystem* System, UNiagaraComponent* SourceComponent, UClass* ParentClass)
{
	//TODO: This is tinkering with BP internals more than I'd like. Would be nice to hide these away into a BP utilities class.
	if (System == nullptr || ParentClass == nullptr)
	{
		return nullptr;
	}

	UBlueprint* NewBP = CreateNewBPAsset_Internal(NewAssetPath, ParentClass);
	if (NewBP == nullptr || NewBP->SkeletonGeneratedClass == nullptr)
	{
		Error(TEXT("Failed to create new Niagara Blueprint."));
		return nullptr;
	}
	
	UEdGraph* Graph = GetConstructionScriptGraph(NewBP);
	if (Graph == nullptr)
	{
		Error(TEXT("Failed to find construction script."));
		return nullptr;
	}

	FString NiagaraCompName(TEXT("Niagara"));
	UActorComponent* NewComp = AddComponent_Internal(NewBP, UNiagaraComponent::StaticClass(), NiagaraCompName);
	UNiagaraComponent* NiagaraComp = Cast<UNiagaraComponent>(NewComp);
	if (NiagaraComp == nullptr)
	{
		Error(TEXT("Failed to create new Niagara Component."));
		return nullptr;
	}

	NiagaraComp->SetAsset(System);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();

	//Find the first Then pin to hook up to.
	TArray<UK2Node_FunctionEntry*> Entries;
	Graph->GetNodesOfClass(Entries);
	UK2Node_FunctionEntry* ScriptEntry = Entries.Num() > 0 ? Entries[0] : nullptr;
	if (ScriptEntry == nullptr)
	{
		Error(TEXT("Unknown Error creating Blueprint. Could not find Entry node."));
		return nullptr;
	}
	UEdGraphPin* ThenPin = ScriptEntry->FindPin(UEdGraphSchema_K2::PN_Then, EGPD_Output);
	
	int32 NodePosX = ScriptEntry->NodePosX;
	int32 NodePosY = ScriptEntry->NodePosY;

	//Now iterate over all User Parameters and mirror them in BP vars.

	FNiagaraExternalEditContext Context(System);
	FNiagaraExt_UserVariables UserVariables;
	UNiagaraExternalEditUtilities::GetUserVariables(System, UserVariables, Context);
	Error(Context.Errors);

	for (const FNiagaraExt_UserVariable& UserVariable : UserVariables.UserVariables)
	{
		if (ThenPin == nullptr)
		{
			Error(TEXT("Failed to find Then Pin to connect variables."));
			break;
		}

		FName VarName = UserVariable.Name;
		FNiagaraTypeDefinition VarType = UserVariable.Type;
		FNiagaraVariableBase NiagaraVar(VarType, VarName);
		FNiagaraVariant DefaultValueVariant;
		if (SourceComponent)
		{
			SourceComponent->GetVariable_InternalUseOnly(NiagaraVar, DefaultValueVariant);
		}

		//Use the system default if we didn't read a valid value from the component.
		if(DefaultValueVariant.IsValid(VarType) == false)
		{
			UserVariable.DefaultValue.Get(DefaultValueVariant, Context);
		}

		Error(Context.Errors);

		//Create a new BP Variable with the same name, type and tooltip.
		FString DefaultValue;
		NiagaraSchema->TryGetPinDefaultValueFromNiagaraVariant(NiagaraVar, DefaultValueVariant, DefaultValue);
		FEdGraphPinType PinType = FNiagaraBlueprintUtil::TypeDefinitionToBlueprintType(VarType);
		if (!AddVariable_Internal(NewBP, VarName, PinType, UserVariable.Description.ToString(), DefaultValue))
		{
			Error(TEXT("Could not add variable for Niagara User Parameter: {0}"), *UserVariable.Name.ToString());
			continue;
		}
	
		NodePosX += 600;

		//Now add a SetVariables node to initialize the variable to the given corresponding BP variable.
		UFunction* SetVariableFunc = GetSetVariablesFunctionForType(VarType);
		if (!SetVariableFunc)
		{
			Error(TEXT("Could not find SetVariable function for type: {0}"), *VarType.GetNameText().ToString());
			continue;
		}

		//Create the call node to set this variable.
		FGraphNodeCreator<UK2Node_CallFunction> NodeCreator(*Graph);
		UK2Node_CallFunction* SetVariablesCallNode = NodeCreator.CreateNode();
		SetVariablesCallNode->NodePosX = NodePosX;
		SetVariablesCallNode->NodePosY = NodePosY;
		SetVariablesCallNode->SetFromFunction(SetVariableFunc);
		NodeCreator.Finalize();
		//SetVariablesCallNode->AllocateDefaultPins();

		//Setup it's pins to the correct component, variable name and link to the BP variable above.
		UEdGraphPin* FuncCallSelfPin = K2Schema->FindSelfPin(*SetVariablesCallNode, EGPD_Input);
		UEdGraphPin* FuncCallVariableNamePin = SetVariablesCallNode->FindPin(TEXT("InVariableName"), EGPD_Input);
		UEdGraphPin* FuncCallValuePin = SetVariablesCallNode->FindPin(TEXT("InValue"), EGPD_Input);
		UEdGraphPin* FuncCallExecPin = K2Schema->FindExecutionPin(*SetVariablesCallNode, EGPD_Input);
		UEdGraphPin* FuncCallThenPin = K2Schema->FindExecutionPin(*SetVariablesCallNode, EGPD_Output);

		//If we can't find the value pin by name, look for a single matching pin by type. If there are multiple then bail.
		if (FuncCallValuePin == nullptr)
		{		
			bool bMultipleError = false;
			for (UEdGraphPin* Pin : SetVariablesCallNode->Pins)
			{
				if (Pin == FuncCallSelfPin || Pin == FuncCallVariableNamePin || Pin == FuncCallExecPin || Pin == FuncCallThenPin)
				{
					continue;
				}

				UEdGraphPin* MatchedPin = nullptr;
				if (Pin->PinType == PinType)
				{
					MatchedPin = Pin;
				}
				else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Object && Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
				{
					//If we're an object type lets just try find any acceptable parent too.
					UClass* SrcClass = Cast<UClass>(PinType.PinSubCategoryObject);
					UClass* DstClass = Cast<UClass>(Pin->PinType.PinSubCategoryObject);
					if (SrcClass && DstClass && SrcClass->IsChildOf(DstClass))
					{
						MatchedPin = Pin;
					}
				}

				if (MatchedPin)
				{
					if (FuncCallValuePin)
					{
						bMultipleError = true;
						break;
					}
					FuncCallValuePin = MatchedPin;
				}				
			}

			//If we have multiple of the input type then we can't tell which to use. 
			//Error out.
			if (bMultipleError)
			{
				FuncCallValuePin = nullptr;
			}
		}

		if (!FuncCallSelfPin || !FuncCallVariableNamePin || !FuncCallValuePin || !FuncCallExecPin || !FuncCallThenPin)
		{
			Error(TEXT("Failed to create Set Variables Node for User Variable: {0}"), *UserVariable.Name.ToString());
			continue;
		}

		//Create a get node for our newly created BP Variable.
		FGraphNodeCreator<UK2Node_VariableGet> VarGetNodeCreator(*Graph);
		UK2Node_VariableGet* VarGetNode = VarGetNodeCreator.CreateNode();
		VarGetNode->NodePosX = NodePosX - 200;
		VarGetNode->NodePosY = NodePosY + 300;

		UEdGraphSchema_K2::ConfigureVarNode(VarGetNode, VarName, NewBP->SkeletonGeneratedClass, NewBP);
		VarGetNodeCreator.Finalize();
		//VarGetNode->AllocateDefaultPins();

		UEdGraphPin* VarNodeValuePin = VarGetNode->GetValuePin();
		if (VarNodeValuePin == nullptr)
		{
			Error(TEXT("Failed to create Get BP Variable node for User Variable: {0}"), *UserVariable.Name.ToString());
			continue;
		}

		//Create a get node for our newly created Niagara Component
		FGraphNodeCreator<UK2Node_VariableGet> CompGetNodeCreator(*Graph);
		UK2Node_VariableGet* CompGetNode = CompGetNodeCreator.CreateNode();
		CompGetNode->NodePosX = NodePosX - 200;
		CompGetNode->NodePosY = NodePosY + 150;

		UEdGraphSchema_K2::ConfigureVarNode(CompGetNode, *NiagaraCompName, NewBP->SkeletonGeneratedClass, NewBP);
		CompGetNodeCreator.Finalize();
		//CompGetNode->AllocateDefaultPins();

		UEdGraphPin* CompNodeValuePin = CompGetNode->GetValuePin();
		if (CompNodeValuePin == nullptr)
		{
			Error(TEXT("Failed to create Get BP Component node for User Variable: {0}"), *UserVariable.Name.ToString());
			continue;
		}
		
		//Now lets connect them all up.
		K2Schema->TryCreateConnection(CompNodeValuePin, FuncCallSelfPin);
		K2Schema->TrySetDefaultValue(*FuncCallVariableNamePin, UserVariable.Name.ToString());
		K2Schema->TryCreateConnection(VarNodeValuePin, FuncCallValuePin);
		K2Schema->TryCreateConnection(ThenPin, FuncCallExecPin);

		//Now move our then pin onto this node
		ThenPin = FuncCallThenPin;
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(NewBP);
	FKismetEditorUtilities::CompileBlueprint(NewBP, EBlueprintCompileOptions::SkipGarbageCollection);

	return NewBP;
}

UFunction* UNiagaraToolset_Blueprint::GetSetVariablesFunctionForType(const FNiagaraTypeDefinition& TypeDef)
{
	if (TypeDef == FNiagaraTypeHelper::GetDoubleDef() || TypeDef == FNiagaraTypeDefinition::GetFloatDef() || TypeDef == FNiagaraTypeDefinition::GetHalfDef())
	{
		return UNiagaraComponent::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraComponent, SetVariableFloat));
	}
	if (TypeDef == FNiagaraTypeHelper::GetVector2DDef() || TypeDef == FNiagaraTypeDefinition::GetVec2Def())
	{
		return UNiagaraComponent::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraComponent, SetVariableVec2));
	}
	if (TypeDef == FNiagaraTypeDefinition::GetPositionDef())
	{
		return UNiagaraComponent::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraComponent, SetVariablePosition));
	}
	if (TypeDef == FNiagaraTypeHelper::GetVectorDef() || TypeDef == FNiagaraTypeDefinition::GetVec3Def())
	{
		return UNiagaraComponent::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraComponent, SetVariableVec3));
	}
	if (TypeDef == FNiagaraTypeHelper::GetVector4Def() || TypeDef == FNiagaraTypeDefinition::GetVec4Def())
	{
		return UNiagaraComponent::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraComponent, SetVariableVec4));
	}
	if (TypeDef == FNiagaraTypeHelper::GetQuatDef() || TypeDef == FNiagaraTypeDefinition::GetQuatDef())
	{
		return UNiagaraComponent::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraComponent, SetVariableQuat));
	}
	if (TypeDef == FNiagaraTypeDefinition::GetColorDef())
	{
		return UNiagaraComponent::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraComponent, SetVariableLinearColor));
	}
	if (TypeDef == FNiagaraTypeDefinition::GetIntDef())
	{
		return UNiagaraComponent::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraComponent, SetVariableInt));
	}
	if (TypeDef == FNiagaraTypeDefinition::GetBoolDef())
	{
		return UNiagaraComponent::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraComponent, SetVariableBool));
	}
	if (TypeDef == FNiagaraTypeDefinition::GetUMaterialDef())
	{
		return UNiagaraComponent::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraComponent, SetVariableMaterial));
	}
	if (TypeDef.IsEnum())//Enum types are written as int32 values currently.
	{
		return UNiagaraComponent::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraComponent, SetVariableInt));
	}
	if (TypeDef.IsUObject())
	{
		return UNiagaraComponent::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraComponent, SetVariableObject));
	}
	return nullptr;
}


 #undef LOCTEXT_NAMESPACE