// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "NiagaraToolset.h"
#include "EdGraph/EdGraphPin.h"
#include "NiagaraToolset_Blueprint.generated.h"

class UFunction;
struct FNiagaraTypeDefinition;
class UEdGraph;
class UBlueprint;
class AActor;
class UActorComponent;
class UNiagaraSystem;
class UNiagaraComponent;

/**
 * Niagara Toolset for Niagara and Blueprint integration.
 *
 * Provides utilities for creating Blueprint wrappers around Niagara Systems,
 * making it easier to expose Niagara effects as reusable Blueprint actors.
 *
 * Use this toolset when you need to create Blueprint-based FX actors from Niagara systems.
 */
UCLASS(Blueprintable)
class UNiagaraToolset_Blueprint : public UNiagaraToolset
{
	GENERATED_BODY()

public:

	/**
	 * Creates a Blueprint actor wrapper around a Niagara System.
	 * This generates a new Blueprint actor with a Niagara component configured to use the specified system.
	 *
	 * Naming convention: NS_MyEffect -> B_MyEffect
	 *
	 * @param NewAssetPath Full path for the new Blueprint asset (prefer same directory as System)
	 * @param System The Niagara System to wrap in the Blueprint
	 * @param ParentClass The parent Actor class for the Blueprint (e.g., AActor)
	 * @return The newly created Blueprint actor asset
	 */
	UFUNCTION(meta = (AICallable), Category = "Util")
	static UBlueprint* ConstructNiagaraBPWrapperFromSystem(const FString& NewAssetPath, UNiagaraSystem* System, UClass* ParentClass);

	/**
	 * Creates a Blueprint actor wrapper from a Niagara Component.
	 * This generates a new Blueprint actor and preserves all component property values and user variable overrides.
	 *
	 * Naming convention: NS_MyEffect -> B_MyEffect
	 *
	 * @param NewAssetPath Full path for the new Blueprint asset (prefer same directory as System)
	 * @param Component The Niagara Component with configured properties and overrides to preserve
	 * @param ParentClass The parent Actor class for the Blueprint (e.g., AActor)
	 * @return The newly created Blueprint actor asset
	 */
	UFUNCTION(meta = (AICallable), Category = "Util")
	static UBlueprint* ConstructNiagaraBPWrapperFromComponent(const FString& NewAssetPath, UNiagaraComponent* Component, UClass* ParentClass);


private:

	/// Internal: Shared logic for Blueprint wrapper construction
	static UBlueprint* ConstructBPWrapper_Internal(const FString& NewAssetPath, UNiagaraSystem* System, UNiagaraComponent* Component, UClass* ParentClass);

	/// Internal: Gets the appropriate SetVariable function for a Niagara type
	static UFunction* GetSetVariablesFunctionForType(const FNiagaraTypeDefinition& TypeDef);

	/// Internal: Creates a new Blueprint asset
	static UBlueprint* CreateNewBPAsset_Internal(const FString& NewAssetPath, UClass* ParentClass);

	/// Internal: Gets the construction script graph from a Blueprint
	static UEdGraph* GetConstructionScriptGraph(UBlueprint* BP);

	/// Internal: Adds a component to a Blueprint
	static UActorComponent* AddComponent_Internal(UBlueprint* BP, TSubclassOf<UActorComponent> ComponentClass, FString ComponentName);

	/// Internal: Adds a variable to a Blueprint
	static bool AddVariable_Internal(UBlueprint* Blueprint, FName Name, FEdGraphPinType Type, FString Tooltip, FString DefaultValue);


};