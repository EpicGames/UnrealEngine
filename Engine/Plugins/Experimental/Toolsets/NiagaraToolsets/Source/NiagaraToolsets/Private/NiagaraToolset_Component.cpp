// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraToolset_Component.h"

#include "NiagaraToolsetsCommon.h"
#include "NiagaraToolsetsSettings.h"

#include "NiagaraSystem.h"
#include "NiagaraComponent.h"
#include "NiagaraParameterStore.h"
#include "NiagaraUserRedirectionParameterStore.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraEditorUtilities.h"

#include "ScopedTransaction.h"

#include "Kismet/KismetSystemLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraToolset_Component)

#define LOCTEXT_NAMESPACE "UNiagaraToolset_Component"

void UNiagaraToolset_Component::SetSystem(UNiagaraComponent* NiagaraComponent, UNiagaraSystem* System, bool bResetExistingOverrideParameters)
{
	if(!ValidateSystem(System)) return;
	if(!ValidateComponent(NiagaraComponent)) return;

	if (System == NiagaraComponent->GetAsset())
	{
		Error(LOCTEXT("SetSameSystem", "New Niagara System is the same as the existing System."));
		return;
	}

	NiagaraComponent->SetAsset(System, bResetExistingOverrideParameters);
}

TArray<FNiagaraExt_VariableInst> UNiagaraToolset_Component::GetUserVariables(UNiagaraComponent* NiagaraComponent)
{
	TArray<FNiagaraExt_VariableInst> VarInstArray;
	if (NiagaraComponent == nullptr)
	{
		Error(LOCTEXT("NullComponent", "Component was nullptr but it is required."));
		return VarInstArray;
	}
	
	const FNiagaraUserRedirectionParameterStore& UserParamStore = NiagaraComponent->GetOverrideParameters();
	TArray<FNiagaraVariable> UserVariables;
	UserParamStore.GetUserParameters(UserVariables);

	VarInstArray.Reserve(UserVariables.Num());
	
	for (const FNiagaraVariable& Var : UserVariables)
	{
		FText ErrorText;
		FNiagaraExt_VariableInst& VarInst = VarInstArray.AddDefaulted_GetRef();
		VarInst.Name = Var.GetName();
		VarInst.Type = Var.GetType();	
		VarInst.Value.Set(Var.GetType(), NiagaraComponent->GetCurrentParameterValue(Var));
	}
	
	return VarInstArray;
}

void UNiagaraToolset_Component::SetVariable(UNiagaraComponent* NiagaraComponent, FNiagaraExt_VariableInst Variable)
{
	if (NiagaraComponent == nullptr)
	{
		Error(LOCTEXT("NullComponent", "Component was nullptr but it is required."));
		return;
	}

	FNiagaraExternalEditContext Context;
	FNiagaraVariant Variant;
	Variable.Value.Get(Variant, Context);
	Error(Context.Errors);
	if(Context.HasErrors())
	{
		return;
	}

	FNiagaraExt_VariableInst VarInst;
	const FScopedTransaction Transaction(LOCTEXT("SetVariable", "AI Assistant Setting a User Variable for a NiagaraComponent."));
	NiagaraComponent->Modify();
	NiagaraComponent->SetVariable_InternalUseOnly(FNiagaraVariableBase(Variable.Type, Variable.Name), Variant);

	FPropertyChangedEvent PropertyChangedEvent(nullptr, EPropertyChangeType::ValueSet);
	NiagaraComponent->PostEditChangeProperty(PropertyChangedEvent);
}

FNiagaraExt_VariableInst UNiagaraToolset_Component::GetVariable(UNiagaraComponent* NiagaraComponent, FNiagaraExt_Variable Var)
{
	FNiagaraExt_VariableInst VarInst;
	if (NiagaraComponent == nullptr)
	{
		Error(LOCTEXT("NullComponent", "Component was nullptr but it is required."));
		return VarInst;
	}

	VarInst.Name = Var.Name;
	VarInst.Type = Var.Type;
	FNiagaraVariableBase Variable(Var.Type, Var.Name);

	FNiagaraVariant Variant;
	NiagaraComponent->GetVariable_InternalUseOnly(Variable, Variant);
	VarInst.Value.Set(Var.Type, Variant);	
	return VarInst;
}

#undef LOCTEXT_NAMESPACE
