// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraToolset.h"
#include "NiagaraExternalSystemEditorUtilities.h"
#include "NiagaraToolset_Component.generated.h"

class UNiagaraComponent;

/**
 * Niagara Toolset for Niagara Component operations.
 *
 * Use this toolset when working with:
 * - Niagara Components on actors
 * - Runtime Niagara system manipulation
 * - User variable overrides on component instances
 * - FX-related operations in levels or blueprints
 *
 * This is the primary toolset for runtime Niagara operations.
 */
UCLASS(Blueprintable)
class UNiagaraToolset_Component : public UNiagaraToolset
{
	GENERATED_BODY()

public:

	/**
	 * Sets the Niagara System for a component.
	 * Use this instead of setting the Asset property directly to ensure proper initialization.
	 *
	 * @param NiagaraComponent The Niagara component to set the system on
	 * @param System The Niagara system asset to assign to the component
	 * @param bResetExistingOverrideParameters If true, reset all user variables to system defaults; if false, preserve matching overrides
	 */
	UFUNCTION(meta = (AICallable), Category = "Component")
	static void SetSystem(UNiagaraComponent* NiagaraComponent, UNiagaraSystem* System, bool bResetExistingOverrideParameters);

	/**
	 * Returns all user variable values currently set on the component.
	 * This retrieves the current values of all user-exposed parameters that can be overridden at the component level.
	 *
	 * @param Component The Niagara component to retrieve user variables from
	 * @return Array of user variable instances with their current values
	 */
	UFUNCTION(meta = (AICallable), Category = "User Variables")
	static TArray<FNiagaraExt_VariableInst> GetUserVariables(UNiagaraComponent* Component);

	/**
	 * Sets the value of a user variable on the component.
	 * This overrides the default value of a user-exposed parameter on a specific component instance.
	 *
	 * @param Component The Niagara component to set the variable on
	 * @param Variable The variable instance containing the name and new value to set
	 */
	UFUNCTION(meta = (AICallable), Category = "User Variables")
	static void SetVariable(UNiagaraComponent* Component, FNiagaraExt_VariableInst Variable);

	/**
	 * Gets the current value of a specific user variable on the component.
	 * This retrieves the current value of a user-exposed parameter, including any component-level overrides.
	 *
	 * @param Component The Niagara component to retrieve the variable from
	 * @param Var The variable definition (name and type) to look up
	 * @return The variable instance containing the current value
	 */
	UFUNCTION(meta = (AICallable), Category = "User Variables")
	static FNiagaraExt_VariableInst GetVariable(UNiagaraComponent* Component, FNiagaraExt_Variable Var);
};
