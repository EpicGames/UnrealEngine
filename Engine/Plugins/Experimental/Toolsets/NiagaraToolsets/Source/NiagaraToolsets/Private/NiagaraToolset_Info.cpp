// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraToolset_Info.h"
#include "NiagaraExternalSystemEditorUtilities.h"

#include "JsonSchema/JsonSchemaGenerator.h"
#include "JsonObjectConverter.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"

#include "NiagaraDataInterfaceSkeletalMesh.h"

#include "NiagaraToolset_System.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraToolset_Info)

#define LOCTEXT_NAMESPACE "UNiagaraToolset_Info"

//Assistant was getting super confused about enum values so needed to give it some help.
//TODO: Investigate why this doesn't just work in core functionality.
FString UNiagaraToolset_Info::UEnum_Info(UEnum* Enum)
{
	if (Enum)
	{
		TSharedPtr<FJsonObject> JsonObj = MakeShared<FJsonObject>();
		JsonObj->SetStringField(TEXT("Name"), Enum->GetName());

		TArray<TSharedPtr<FJsonValue>> EnumValuesArray;
		for (int32 i = 0; i < Enum->NumEnums(); ++i)
		{
			TSharedPtr<FJsonObject> EnumValueObj = MakeShared<FJsonObject>();
			EnumValueObj->SetStringField(TEXT("Name"), Enum->GetNameStringByIndex(i));
			EnumValueObj->SetNumberField(TEXT("Value"), Enum->GetValueByIndex(i));
			EnumValueObj->SetStringField(TEXT("DisplayName"), Enum->GetDisplayNameTextByIndex(i).ToString());
			EnumValueObj->SetStringField(TEXT("Description"), Enum->GetToolTipTextByIndex(i).ToString());
			EnumValuesArray.Add(MakeShared<FJsonValueObject>(EnumValueObj));
		}

		JsonObj->SetArrayField(TEXT("Values"), EnumValuesArray);

		FString EnumJsonStr;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&EnumJsonStr);
		FJsonSerializer::Serialize(JsonObj.ToSharedRef(), Writer);
		return EnumJsonStr;
	}

	return TEXT("{}");
}


#undef LOCTEXT_NAMESPACE
