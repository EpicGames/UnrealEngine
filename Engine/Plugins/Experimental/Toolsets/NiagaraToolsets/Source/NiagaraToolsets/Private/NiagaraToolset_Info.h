// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraToolset.h"
#include "StructUtils/InstancedStruct.h"
#include "NiagaraExternalSystemEditorUtilities.h"

#include "NiagaraToolset_Assets.h"

#include "NiagaraToolset_Info.generated.h"

//////////////////////////////////////////////////////////////////////////

/**
 * Niagara Toolset for general Niagara information and guidance.
 *
 * Provides:
 * - Enum value lookups
 * - General usage information
 *
 * Call these functions when you need context about Niagara types.
 */
UCLASS(Blueprintable)
class UNiagaraToolset_Info : public UNiagaraToolset
{
	GENERATED_BODY()

public:

	/**
	 * Returns information about a UEnum and all its values.
	 * ALWAYS call this when working with a UEnum type to see valid values.
	 */
	UFUNCTION(meta = (AICallable), Category = "")
	static FString UEnum_Info(UEnum* Enum);
};
