// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraToolset_System.h"
#include "NiagaraToolsetsSettings.h"

#include "NiagaraExternalSystemEditorUtilities.h"
#include "NiagaraSystem.h"

#include "Containers/Ticker.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"

#include "ScopedTransaction.h"

#include "Kismet/KismetSystemLibrary.h"

#include "JsonSchema/JsonSchemaGenerator.h"
#include "JsonObjectConverter.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraToolset_System)

#define LOCTEXT_NAMESPACE "UNiagaraToolset_System"

static float NiagaraToolsetCompileWaitTimeoutSeconds = 120.0f;
static FAutoConsoleVariableRef CVarNiagaraToolsetCompileWaitTimeoutSeconds(
	TEXT("NiagaraToolset.CompileWaitTimeoutSeconds"),
	NiagaraToolsetCompileWaitTimeoutSeconds,
	TEXT("Maximum time in seconds that NiagaraToolset tool calls will wait for an in-flight Niagara System compile to complete before returning a timeout error."),
	ECVF_Default);


TArray<UNiagaraScript*> UNiagaraToolset_System::GetAvailableDynamicInputs(const FNiagaraTypeDefinition& Type)
{
	TArray<UNiagaraScript*> Ret;
	FNiagaraExternalEditContext Context;
	UNiagaraExternalEditUtilities::GetAvailableDynamicInputs(Type, Ret, Context);
	Error(Context.Errors);
	return Ret;
}

UNiagaraSystem* UNiagaraToolset_System::CreateNiagaraSystem(FString AssetName, FString AssetPath, UNiagaraSystem* TemplateSystem)
{
	const FString PackageName = (AssetPath / AssetName);
	if (!FPackageName::IsValidLongPackageName(PackageName))
	{
		Error(TEXT("Asset Path and Asset Name do not form a valid package name."));
		return nullptr;
	}

	FNiagaraExternalEditContext Context;
	UNiagaraSystem* NewSystem = UNiagaraExternalEditUtilities::CreateNiagaraSystem(AssetName, AssetPath, TemplateSystem, Context);

	Error(Context.Errors);

	return NewSystem;
}


FNiagaraExt_SystemSchema UNiagaraToolset_System::GetSystemSchema()
{
	FNiagaraExt_SystemSchema Schema;
	UNiagaraExternalEditUtilities::GetSystemSchema(Schema);
	return Schema;
}

FNiagaraExt_EmitterSchema UNiagaraToolset_System::GetEmitterSchema()
{
	FNiagaraExt_EmitterSchema Schema;
	UNiagaraExternalEditUtilities::GetEmitterSchema(Schema);
	return Schema;
}

FNiagaraExt_RendererSchema UNiagaraToolset_System::GetRendererSchema(TSubclassOf<UNiagaraRendererProperties> RendererClass)
{
	FNiagaraExt_RendererSchema Schema;
	UNiagaraExternalEditUtilities::GetRendererSchema(RendererClass, Schema);
	return Schema;
}

FNiagaraExt_DataInterfaceSchema UNiagaraToolset_System::GetDataInterfaceSchema(TSubclassOf<UNiagaraDataInterface> DataInterfaceClass)
{
	FNiagaraExt_DataInterfaceSchema Schema;
	UNiagaraExternalEditUtilities::GetDataInterfaceSchema(DataInterfaceClass, Schema);
	return Schema;
}

FNiagaraExt_StackInputSchema UNiagaraToolset_System::GetStackInputSchema(const FNiagaraExt_StackItemReference& InputReference)
{
	FNiagaraExternalEditContext Context(InputReference);
	FNiagaraExt_StackInputSchema Schema;
	UNiagaraExternalEditUtilities::GetStackInputSchema(InputReference, Schema, Context);
	Error(Context.Errors);
	return Schema;
}

FNiagaraExt_ModuleSchema UNiagaraToolset_System::GetModuleSchema(const FNiagaraExt_StackItemReference& ModuleReference)
{
	FNiagaraExternalEditContext Context(ModuleReference);
	FNiagaraExt_ModuleSchema Schema;
	UNiagaraExternalEditUtilities::GetModuleSchema(ModuleReference, Schema, Context);
	Error(Context.Errors);
	return Schema;
}

FNiagaraExt_DynamicInputSchema UNiagaraToolset_System::GetDynamicInputSchema(const FNiagaraExt_StackItemReference& DynamicInputReference)
{
	FNiagaraExternalEditContext Context(DynamicInputReference);
	FNiagaraExt_DynamicInputSchema Schema;
	UNiagaraExternalEditUtilities::GetDynamicInputSchema(DynamicInputReference, Schema, Context);
	Error(Context.Errors);
	return Schema;
}

FNiagaraExt_ModuleSchema UNiagaraToolset_System::GetModuleSchemaFromAsset(const UNiagaraScript* ModuleAsset)
{
	FNiagaraExternalEditContext Context;
	FNiagaraExt_ModuleSchema Schema;
	UNiagaraExternalEditUtilities::GetModuleSchema(ModuleAsset, Schema, Context);
	Error(Context.Errors);
	return Schema;
}

FNiagaraExt_DynamicInputSchema UNiagaraToolset_System::GetDynamicInputSchemaFromAsset(const UNiagaraScript* DynamicInputAsset)
{
	FNiagaraExternalEditContext Context;
	FNiagaraExt_DynamicInputSchema Schema;
	UNiagaraExternalEditUtilities::GetDynamicInputSchema(DynamicInputAsset, Schema, Context);
	Error(Context.Errors);
	return Schema;
}

//////////////////////////////////////////////////////////////////////////
// Summary tier

FNiagaraExt_SystemSummary UNiagaraToolset_System::GetSystemSummary(UNiagaraSystem* System)
{
	FNiagaraExternalEditContext Context(System);
	FNiagaraExt_SystemSummary Summary;
	UNiagaraExternalEditUtilities::GetSystemSummary(System, Summary, Context);
	Error(Context.Errors);
	return Summary;
}

FNiagaraExt_EmitterSummary UNiagaraToolset_System::GetEmitterSummary(const FNiagaraExt_StackItemReference& EmitterRef)
{
	FNiagaraExternalEditContext Context(EmitterRef);
	FNiagaraExt_EmitterSummary Summary;
	UNiagaraExternalEditUtilities::GetEmitterSummary(EmitterRef, Summary, Context);
	Error(Context.Errors);
	return Summary;
}

//////////////////////////////////////////////////////////////////////////
// Topology tier

FNiagaraExt_EmitterTopology UNiagaraToolset_System::GetEmitterTopology(const FNiagaraExt_StackItemReference& EmitterRef)
{
	FNiagaraExternalEditContext Context(EmitterRef);
	FNiagaraExt_EmitterTopology Topology;
	UNiagaraExternalEditUtilities::GetEmitterTopology(EmitterRef, Topology, Context);
	Error(Context.Errors);
	return Topology;
}

FNiagaraExt_ScriptStackTopology UNiagaraToolset_System::GetScriptStackTopology(const FNiagaraExt_StackItemReference& ScriptRef)
{
	FNiagaraExternalEditContext Context(ScriptRef);
	FNiagaraExt_ScriptStackTopology Topology;
	UNiagaraExternalEditUtilities::GetScriptStackTopology(ScriptRef, Topology, Context);
	Error(Context.Errors);
	return Topology;
}

FNiagaraExt_ModuleTopology UNiagaraToolset_System::GetModuleTopology(const FNiagaraExt_StackItemReference& ModuleRef)
{
	FNiagaraExternalEditContext Context(ModuleRef);
	FNiagaraExt_ModuleTopology Topology;
	UNiagaraExternalEditUtilities::GetModuleTopology(ModuleRef, Topology, Context);
	Error(Context.Errors);
	return Topology;
}

FNiagaraExt_StackInputTopology UNiagaraToolset_System::GetStackInputTopology(const FNiagaraExt_StackItemReference& StackInputRef)
{
	FNiagaraExternalEditContext Context(StackInputRef);
	FNiagaraExt_StackInputTopology Topology;
	UNiagaraExternalEditUtilities::GetStackInputTopology(StackInputRef, Topology, Context);
	Error(Context.Errors);
	return Topology;
}

//////////////////////////////////////////////////////////////////////////
// Data layer

FNiagaraExt_UserVariables UNiagaraToolset_System::GetUserVariables(UNiagaraSystem* System)
{
	FNiagaraExternalEditContext Context(System);
	FNiagaraExt_UserVariables Variables;
	UNiagaraExternalEditUtilities::GetUserVariables(System, Variables, Context);
	Error(Context.Errors);
	return Variables;
}

FNiagaraExt_SystemData UNiagaraToolset_System::GetSystemData(UNiagaraSystem* System)
{
	FNiagaraExternalEditContext Context(System);
	FNiagaraExt_SystemData Data;
	UNiagaraExternalEditUtilities::GetSystemData(System, Data, Context);
	Error(Context.Errors);
	return Data;
}

FNiagaraExt_EmitterData UNiagaraToolset_System::GetEmitterData(const FNiagaraExt_StackItemReference& EmitterRef)
{
	FNiagaraExternalEditContext Context(EmitterRef);
	FNiagaraExt_EmitterData Data;
	UNiagaraExternalEditUtilities::GetEmitterData(EmitterRef, Data, Context);
	Error(Context.Errors);
	return Data;
}

FNiagaraExt_RendererData UNiagaraToolset_System::GetRendererData(const FNiagaraExt_StackItemReference& RendererRef)
{
	FNiagaraExternalEditContext Context(RendererRef);
	FNiagaraExt_RendererData Data;
	UNiagaraExternalEditUtilities::GetRendererData(RendererRef, Data, Context);
	Error(Context.Errors);
	return Data;
}

FNiagaraExt_StackInputValue UNiagaraToolset_System::GetStackInputData(const FNiagaraExt_StackItemReference& StackInputRef)
{
	FNiagaraExternalEditContext Context(StackInputRef);
	FNiagaraExt_StackInputValue Data;
	UNiagaraExternalEditUtilities::GetStackInputData(StackInputRef, Data, Context);
	Error(Context.Errors);
	return Data;
}

TArray<FNiagaraExt_ModuleInputValues> UNiagaraToolset_System::GetEmitterInputValues(const FNiagaraExt_StackItemReference& EmitterRef)
{
	FNiagaraExternalEditContext Context(EmitterRef);
	TArray<FNiagaraExt_ModuleInputValues> Values;
	UNiagaraExternalEditUtilities::GetEmitterInputValues(EmitterRef, Values, Context);
	Error(Context.Errors);
	return Values;
}

TArray<FNiagaraExt_ModuleInputValues> UNiagaraToolset_System::GetScriptStackInputValues(const FNiagaraExt_StackItemReference& ScriptRef)
{
	FNiagaraExternalEditContext Context(ScriptRef);
	TArray<FNiagaraExt_ModuleInputValues> Values;
	UNiagaraExternalEditUtilities::GetScriptStackInputValues(ScriptRef, Values, Context);
	Error(Context.Errors);
	return Values;
}

FNiagaraExt_ModuleInputValues UNiagaraToolset_System::GetModuleInputValues(const FNiagaraExt_StackItemReference& ModuleRef)
{
	FNiagaraExternalEditContext Context(ModuleRef);
	FNiagaraExt_ModuleInputValues Values;
	UNiagaraExternalEditUtilities::GetModuleInputValues(ModuleRef, Values, Context);
	Error(Context.Errors);
	return Values;
}

FNiagaraExt_DynamicInputChainRef UNiagaraToolset_System::GetDynamicInputChain(const FNiagaraExt_StackItemReference& StackInputRef)
{
	FNiagaraExternalEditContext Context(StackInputRef);
	FNiagaraExt_DynamicInputChainRef Chain;
	UNiagaraExternalEditUtilities::GetDynamicInputChain(StackInputRef, Chain, Context);
	Error(Context.Errors);
	return Chain;
}

FNiagaraExt_SystemDependencies UNiagaraToolset_System::GetSystemDependencies(UNiagaraSystem* System)
{
	FNiagaraExternalEditContext Context(System);
	FNiagaraExt_SystemDependencies Dependencies;
	UNiagaraExternalEditUtilities::GetSystemDependencies(System, Dependencies, Context);
	Error(Context.Errors);
	return Dependencies;
}

//////////////////////////////////////////////////////////////////////////

void UNiagaraToolset_System::SetSystemData(UNiagaraSystem* System, const FNiagaraExt_SystemData& SystemData)
{
	FNiagaraExternalEditContext Context(System);
	UNiagaraExternalEditUtilities::SetSystemData(System, SystemData, Context);
	Error(Context.Errors);
}

void UNiagaraToolset_System::SetEmitterData(const FNiagaraExt_StackItemReference& Emitter, const FNiagaraExt_EmitterData& EmitterData)
{
	FNiagaraExternalEditContext Context(Emitter);
	UNiagaraExternalEditUtilities::SetEmitterData(Emitter, EmitterData, Context);
	Error(Context.Errors);
}

void UNiagaraToolset_System::SetRendererData(const FNiagaraExt_StackItemReference& Renderer, const FNiagaraExt_RendererData& RendererData)
{
	FNiagaraExternalEditContext Context(Renderer);
	UNiagaraExternalEditUtilities::SetRendererData(Renderer, RendererData, Context);
	Error(Context.Errors);
}

FNiagaraExt_EmitterTopology UNiagaraToolset_System::AddEmitter(UNiagaraSystem* System, UNiagaraEmitter* TemplateEmitter, FName EmitterName)
{
	FNiagaraExternalEditContext Context(System);
	FNiagaraExt_EmitterTopology Topology;
	UNiagaraExternalEditUtilities::AddEmitter(TemplateEmitter, EmitterName, Topology, Context);
	Error(Context.Errors);
	return Topology;
}

void UNiagaraToolset_System::RemoveEmitter(const FNiagaraExt_StackItemReference& EmitterToRemove)
{
	FNiagaraExternalEditContext Context(EmitterToRemove);
	UNiagaraExternalEditUtilities::RemoveEmitter(EmitterToRemove, Context);
	Error(Context.Errors);
}

FNiagaraExt_RendererRef UNiagaraToolset_System::AddRenderer(const FNiagaraExt_StackItemReference& NewRendererLocation, const TSubclassOf<UNiagaraRendererProperties> RendererClass)
{
	FNiagaraExternalEditContext Context(NewRendererLocation);
	FNiagaraExt_RendererRef Ref;
	UNiagaraExternalEditUtilities::AddRenderer(NewRendererLocation, RendererClass, Ref, Context);
	Error(Context.Errors);
	return Ref;
}

void UNiagaraToolset_System::RemoveRenderer(const FNiagaraExt_StackItemReference& RendererToRemove)
{
	FNiagaraExternalEditContext Context(RendererToRemove);
	UNiagaraExternalEditUtilities::RemoveRenderer(RendererToRemove, Context);
	Error(Context.Errors);
}

FNiagaraExt_ModuleTopology UNiagaraToolset_System::AddModule(const FNiagaraExt_StackItemReference& ModuleLocationRef, const UNiagaraScript* ModuleAsset)
{
	FNiagaraExternalEditContext Context(ModuleLocationRef);
	FNiagaraExt_ModuleTopology Topology;
	UNiagaraExternalEditUtilities::AddModule(ModuleLocationRef, ModuleAsset, Topology, Context);
	Error(Context.Errors);
	return Topology;
}

void UNiagaraToolset_System::RemoveModule(const FNiagaraExt_StackItemReference& ModuleToRemove)
{
	FNiagaraExternalEditContext Context(ModuleToRemove);
	UNiagaraExternalEditUtilities::RemoveModule(ModuleToRemove, Context);
	Error(Context.Errors);
}

void UNiagaraToolset_System::SetModuleEnabled(const FNiagaraExt_StackItemReference& ModuleRef, bool bEnabled)
{
	FNiagaraExternalEditContext Context(ModuleRef);
	UNiagaraExternalEditUtilities::SetModuleEnabled(ModuleRef, bEnabled, Context);
	Error(Context.Errors);
}

FNiagaraExt_ModuleTopology UNiagaraToolset_System::AddSetParametersModule(const FNiagaraExt_StackItemReference& ModuleLocationRef, const TArray<FNiagaraExt_SetParameterEntry>& Parameters)
{
	FNiagaraExternalEditContext Context(ModuleLocationRef);
	FNiagaraExt_ModuleTopology Topology;
	UNiagaraExternalEditUtilities::AddSetParametersModule(ModuleLocationRef, Parameters, Topology, Context);
	Error(Context.Errors);
	return Topology;
}

FNiagaraExt_ModuleTopology UNiagaraToolset_System::AddSetParameterEntry(const FNiagaraExt_StackItemReference& ModuleRef, const FNiagaraExt_SetParameterEntry& Entry)
{
	FNiagaraExternalEditContext Context(ModuleRef);
	FNiagaraExt_ModuleTopology Topology;
	UNiagaraExternalEditUtilities::AddSetParameterEntry(ModuleRef, Entry, Topology, Context);
	Error(Context.Errors);
	return Topology;
}

void UNiagaraToolset_System::RemoveSetParameterEntry(const FNiagaraExt_StackItemReference& ModuleRef, FName ParameterName)
{
	FNiagaraExternalEditContext Context(ModuleRef);
	UNiagaraExternalEditUtilities::RemoveSetParameterEntry(ModuleRef, ParameterName, Context);
	Error(Context.Errors);
}

void UNiagaraToolset_System::AddUserVariables(UNiagaraSystem* System, const TArray<FNiagaraExt_UserVariable>& VariablesToAdd)
{
	FNiagaraExternalEditContext Context(System);
	for (const FNiagaraExt_UserVariable& VarToAdd : VariablesToAdd)
	{
		if (!VarToAdd.DefaultValue.IsValid())
		{
			const int32 TypeSize = VarToAdd.Type.GetSize();
			if (TypeSize > 0)
			{
				FNiagaraExt_UserVariable VarWithDefault = VarToAdd;
				TArray<uint8> ZeroBytes;
				ZeroBytes.SetNumZeroed(TypeSize);
				FNiagaraVariant ZeroVariant;
				ZeroVariant.SetBytes(ZeroBytes.GetData(), TypeSize);
				VarWithDefault.DefaultValue.Set(VarToAdd.Type, ZeroVariant);
				UNiagaraExternalEditUtilities::AddUserVariable(System, VarWithDefault, Context);
			}
			else
			{
				Context.Error(FText::Format(
					LOCTEXT("AddUserVariables_ZeroSizeType", "Cannot add user variable '{0}': type '{1}' has no fixed size (object or interface type) and no default value was provided. Specify an explicit default value."),
					FText::FromName(VarToAdd.Name),
					FText::FromString(VarToAdd.Type.GetName())));
			}
		}
		else
		{
			UNiagaraExternalEditUtilities::AddUserVariable(System, VarToAdd, Context);
		}
	}
	Error(Context.Errors);
}

void UNiagaraToolset_System::RemoveUserVariables(UNiagaraSystem* System, const TArray<FNiagaraExt_Variable>& VariablesToRemove)
{
	FNiagaraExternalEditContext Context(System);
	for(auto& VarToRemove : VariablesToRemove)
	{
		UNiagaraExternalEditUtilities::RemoveUserVariable(System, VarToRemove, Context);
	}
	Error(Context.Errors);
}

FNiagaraExt_StackInputValue UNiagaraToolset_System::SetStackInputData(const FNiagaraExt_StackItemReference& InputItemRef, const FNiagaraExt_StackInputValue& InputData)
{
	FNiagaraExternalEditContext Context(InputItemRef);
	UNiagaraExternalEditUtilities::SetStackInputData(InputItemRef, InputData, Context);
	Error(Context.Errors);
	FNiagaraExt_StackInputValue Result;
	UNiagaraExternalEditUtilities::GetStackInputData(InputItemRef, Result, Context);
	return Result;
}

//////////////////////////////////////////////////////////////////////////
// Diagnostics Layer

namespace
{
	// Flattens a Context.Errors array into a single newline-delimited FString so it can be
	// carried through UToolCallAsyncResult::SetError. Empty when there are no errors.
	FString FormatContextErrors(const TArray<FText>& Errors)
	{
		if (Errors.IsEmpty())
		{
			return FString();
		}
		TArray<FString> Lines;
		Lines.Reserve(Errors.Num());
		for (const FText& Err : Errors)
		{
			Lines.Add(Err.ToString());
		}
		return FString::Join(Lines, TEXT("\n"));
	}

	// Invokes OnDone once the system has no VM compile pending, once the system is destroyed,
	// or once TimeoutSeconds has elapsed — whichever comes first.
	//
	// On success: OnDone(LiveSystem, "") where LiveSystem is guaranteed non-null.
	// On failure: OnDone(MaybeSystem, ErrorReason). MaybeSystem may be null (destroyed case).
	//
	// Polls each tick rather than binding OnSystemCompiled. We don't own Niagara's broadcast
	// contract — a cancelled compile, hot reload, or subsystem teardown could leave a delegate-
	// bound async result hanging forever.
	void RunWhenCompileComplete(
		UNiagaraSystem* System,
		TFunction<void(UNiagaraSystem* /*LiveSystem*/, const FString& /*Error*/)>&& OnDone)
	{
		if (System == nullptr)
		{
			OnDone(nullptr, TEXT("System is null."));
			return;
		}

		const bool bBusy = System->HasActiveCompilations()
			|| System->HasOutstandingCompilationRequests(/*bIncludingGPUShaders=*/false);
		if (!bBusy)
		{
			OnDone(System, FString());
			return;
		}

		TWeakObjectPtr<UNiagaraSystem> WeakSystem(System);

		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
			[WeakSystem, OnDone = MoveTemp(OnDone), StartTime = FPlatformTime::Seconds()](float) mutable -> bool
			{
				UNiagaraSystem* LiveSystem = WeakSystem.Get();
				if (LiveSystem == nullptr)
				{
					OnDone(nullptr, TEXT("System was destroyed before compile completed."));
					return false;
				}

				if (LiveSystem->HasActiveCompilations()
					|| LiveSystem->HasOutstandingCompilationRequests(/*bIncludingGPUShaders=*/false))
				{
					const double Timeout = NiagaraToolsetCompileWaitTimeoutSeconds;
					if (FPlatformTime::Seconds() - StartTime > Timeout)
					{
						OnDone(LiveSystem, FString::Printf(
							TEXT("Timed out after %.0f seconds waiting for Niagara System compile to complete."),
							Timeout));
						return false;
					}
					return true;
				}

				OnDone(LiveSystem, FString());
				return false;
			}));
	}

	// Collects stack issues synchronously into OutIssues. Must be called with the system in a
	// no-compile-in-flight state; the utility rejects mid-compile calls.
	void CollectStackIssuesForAsync(UNiagaraSystem* System, FNiagaraExt_StackIssues& OutIssues, FString& OutErrorText)
	{
		FNiagaraExternalEditContext Context(System);
		UNiagaraExternalEditUtilities::GetStackIssues(System, OutIssues, Context);
		OutErrorText = FormatContextErrors(Context.Errors);
	}

	// Collects compile state synchronously into OutState. Safe to call at any time (the utility
	// never blocks), but the async wrapper still gates on compile completion so post-compile
	// state is reported consistently with the other diagnostics functions.
	void CollectCompileStateForAsync(UNiagaraSystem* System, FNiagaraExt_SystemCompileState& OutState, FString& OutErrorText)
	{
		FNiagaraExternalEditContext Context(System);
		UNiagaraExternalEditUtilities::GetSystemCompileState(System, OutState, Context);
		OutErrorText = FormatContextErrors(Context.Errors);
	}
}

UNiagaraToolset_AsyncSystemCompileState* UNiagaraToolset_System::GetSystemCompileState(UNiagaraSystem* System)
{
	UNiagaraToolset_AsyncSystemCompileState* Result = NewObject<UNiagaraToolset_AsyncSystemCompileState>();

	if (System == nullptr)
	{
		Result->SetError(TEXT("System is null."));
		return Result;
	}

	TStrongObjectPtr<UNiagaraToolset_AsyncSystemCompileState> StrongResult(Result);

	RunWhenCompileComplete(System,
		[StrongResult = MoveTemp(StrongResult)](UNiagaraSystem* LiveSystem, const FString& Error) mutable
		{
			if (!Error.IsEmpty())
			{
				StrongResult->SetError(Error);
				return;
			}

			FNiagaraExt_SystemCompileState State;
			FString ErrorText;
			CollectCompileStateForAsync(LiveSystem, State, ErrorText);

			if (!ErrorText.IsEmpty())
			{
				StrongResult->SetError(ErrorText);
				return;
			}
			StrongResult->SetValue(State);
		});

	return Result;
}

UNiagaraToolset_AsyncStackIssues* UNiagaraToolset_System::GetStackIssues(UNiagaraSystem* System)
{
	UNiagaraToolset_AsyncStackIssues* Result = NewObject<UNiagaraToolset_AsyncStackIssues>();

	if (System == nullptr)
	{
		Result->SetError(TEXT("System is null."));
		return Result;
	}

	TStrongObjectPtr<UNiagaraToolset_AsyncStackIssues> StrongResult(Result);

	RunWhenCompileComplete(System,
		[StrongResult = MoveTemp(StrongResult)](UNiagaraSystem* LiveSystem, const FString& Error) mutable
		{
			if (!Error.IsEmpty())
			{
				StrongResult->SetError(Error);
				return;
			}

			FNiagaraExt_StackIssues Issues;
			FString ErrorText;
			CollectStackIssuesForAsync(LiveSystem, Issues, ErrorText);

			if (!ErrorText.IsEmpty())
			{
				StrongResult->SetError(ErrorText);
				return;
			}
			StrongResult->SetValue(Issues);
		});

	return Result;
}

UNiagaraToolset_AsyncApplyStackIssueFixResult* UNiagaraToolset_System::ApplyStackIssueFix(UNiagaraSystem* System, FString IssueId, FString FixId)
{
	UNiagaraToolset_AsyncApplyStackIssueFixResult* Result = NewObject<UNiagaraToolset_AsyncApplyStackIssueFixResult>();

	if (System == nullptr)
	{
		Result->SetError(TEXT("System is null."));
		return Result;
	}

	TStrongObjectPtr<UNiagaraToolset_AsyncApplyStackIssueFixResult> StrongResult(Result);

	// Step 1: wait for any pre-fix compile. Then execute the fix. Then wait for the post-fix
	// compile (if the fix triggered one). Then collect PostFixIssues. Then resolve.
	RunWhenCompileComplete(System,
		[StrongResult, IssueId = MoveTemp(IssueId), FixId = MoveTemp(FixId)](UNiagaraSystem* LiveSystem, const FString& Error) mutable
		{
			if (!Error.IsEmpty())
			{
				StrongResult->SetError(Error);
				return;
			}

			// Apply — engine utility fills bApplied + AppliedFixDescription.
			FNiagaraExt_ApplyStackIssueFixResult ApplyResult;
			FString ApplyErrorText;
			{
				FNiagaraExternalEditContext Context(LiveSystem);
				UNiagaraExternalEditUtilities::ApplyStackIssueFix(LiveSystem, IssueId, FixId, ApplyResult, Context);
				ApplyErrorText = FormatContextErrors(Context.Errors);
			}

			if (!ApplyErrorText.IsEmpty())
			{
				// Issue-not-found, ambiguous, Link-style, unbound delegate — all report via
				// Context.Errors and leave bApplied == false. Surface the error text on the
				// async result.
				StrongResult->SetError(ApplyErrorText);
				return;
			}

			// Step 2: wait for the compile (if any) that the fix just kicked off, then
			// collect PostFixIssues into the composite toolset result.
			RunWhenCompileComplete(LiveSystem,
				[StrongResult = MoveTemp(StrongResult), ApplyResult = MoveTemp(ApplyResult)](UNiagaraSystem* PostFixSystem, const FString& PostError) mutable
				{
					if (!PostError.IsEmpty())
					{
						StrongResult->SetError(PostError);
						return;
					}

					FNiagaraToolset_ApplyStackIssueFixResult Composite;
					Composite.ApplyResult = MoveTemp(ApplyResult);

					FString IssuesErrorText;
					CollectStackIssuesForAsync(PostFixSystem, Composite.PostFixIssues, IssuesErrorText);

					if (!IssuesErrorText.IsEmpty())
					{
						// Fix itself ran (Composite.ApplyResult.bApplied is true) but the post-fix
						// collection failed. Surface the error so the tool call sees why.
						StrongResult->SetError(IssuesErrorText);
						return;
					}

					StrongResult->SetValue(Composite);
				});
		});

	return Result;
}

#undef LOCTEXT_NAMESPACE


