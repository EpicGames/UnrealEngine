// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraToolset.h"
#include "NiagaraExternalSystemEditorUtilities.h"
#include "ToolsetRegistry/ToolCallAsyncResult.h"
#include "NiagaraToolset_System.generated.h"

UCLASS(BlueprintType)
class UNiagaraToolset_AsyncSystemCompileState : public UToolCallAsyncResult
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "NiagaraToolset")
	bool SetValue(const FNiagaraExt_SystemCompileState& InValue)
	{
		return MaybeBroadcastSuccessfulCompletion(FNiagaraExt_SystemCompileState(InValue), Value);
	}

	UPROPERTY(BlueprintReadOnly, Category = "NiagaraToolset")
	FNiagaraExt_SystemCompileState Value;
};

UCLASS(BlueprintType)
class UNiagaraToolset_AsyncStackIssues : public UToolCallAsyncResult
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "NiagaraToolset")
	bool SetValue(const FNiagaraExt_StackIssues& InValue)
	{
		return MaybeBroadcastSuccessfulCompletion(FNiagaraExt_StackIssues(InValue), Value);
	}

	UPROPERTY(BlueprintReadOnly, Category = "NiagaraToolset")
	FNiagaraExt_StackIssues Value;
};

USTRUCT(BlueprintType)
struct FNiagaraToolset_ApplyStackIssueFixResult
{
	GENERATED_BODY()

	/** Direct outcome of the fix application (bApplied + AppliedFixDescription). */
	UPROPERTY(BlueprintReadOnly, Category = "NiagaraToolset")
	FNiagaraExt_ApplyStackIssueFixResult ApplyResult;

	/** Stack-issue state observed after the fix's compile completed. */
	UPROPERTY(BlueprintReadOnly, Category = "NiagaraToolset")
	FNiagaraExt_StackIssues PostFixIssues;
};

UCLASS(BlueprintType)
class UNiagaraToolset_AsyncApplyStackIssueFixResult : public UToolCallAsyncResult
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "NiagaraToolset")
	bool SetValue(const FNiagaraToolset_ApplyStackIssueFixResult& InValue)
	{
		return MaybeBroadcastSuccessfulCompletion(FNiagaraToolset_ApplyStackIssueFixResult(InValue), Value);
	}

	UPROPERTY(BlueprintReadOnly, Category = "NiagaraToolset")
	FNiagaraToolset_ApplyStackIssueFixResult Value;
};


/**
 * Niagara Toolset for Niagara System operations.
 *
 * Provides comprehensive access to Niagara System editing, including:
 * - System, emitter, and module creation and modification
 * - Schema discovery for understanding structure
 * - Topology and Summary inspection for viewing current configuration
 * - Data access for reading and writing properties
 * - Dependency rollups via GetSystemDependencies
 * - Dynamic-input chain traversal via GetDynamicInputChain
 *
 * This is the primary toolset for working with Niagara Systems in the editor.
 */
UCLASS(Blueprintable)
class UNiagaraToolset_System : public UNiagaraToolset
{
	GENERATED_BODY()

public:

	/**
	 * Returns all available Dynamic Input Module assets compatible with the given type.
	 * Dynamic inputs provide procedural value generation for module parameters.
	 *
	 * @param Type The Niagara type definition to find compatible dynamic inputs for
	 * @return Array of dynamic input script assets that can be used with the specified type
	 */
	UFUNCTION(meta = (AICallable), Category = "Assets")
	static TArray<UNiagaraScript*> GetAvailableDynamicInputs(const FNiagaraTypeDefinition& Type);

	/**
	 * Creates a new Niagara System asset.
	 * The new system will be based on the template system, inheriting its configuration and emitters.
	 *
	 * @param AssetName Name of the new asset (without path or extension)
	 * @param AssetPath Directory path where the new asset will be created
	 * @param TemplateSystem Template system to base the new system on (required)
	 * @return The newly created Niagara System asset if successful, nullptr otherwise
	 */
	UFUNCTION(meta = (AICallable), Category = "System")
	static UNiagaraSystem* CreateNiagaraSystem(FString AssetName, FString AssetPath, UNiagaraSystem* TemplateSystem);


	//////////////////////////////////////////////////////////////////////////
	// Schema Layer - Discover structure and available properties

	/**
	 * Returns property schema for Niagara System.
	 * Describes all available properties and their types that can be set on a Niagara System.
	 *
	 * @return Schema describing all system-level properties
	 */
	UFUNCTION(meta = (AICallable), Category = "Schema")
	static FNiagaraExt_SystemSchema GetSystemSchema();

	/**
	 * Returns property schema for Niagara Emitter.
	 * Describes all available properties and their types that can be set on a Niagara Emitter.
	 *
	 * @return Schema describing all emitter-level properties
	 */
	UFUNCTION(meta = (AICallable), Category = "Schema")
	static FNiagaraExt_EmitterSchema GetEmitterSchema();

	/**
	 * Returns property schema for a specific Renderer class.
	 * Describes all available properties and their types for the given renderer type.
	 *
	 * @param RendererClass The renderer class to get the schema for (e.g., UNiagaraSpriteRendererProperties)
	 * @return Schema describing all properties for the specified renderer class
	 */
	UFUNCTION(meta = (AICallable), Category = "Schema")
	static FNiagaraExt_RendererSchema GetRendererSchema(TSubclassOf<UNiagaraRendererProperties> RendererClass);

	/**
	 * Returns property schema for a specific Data Interface class.
	 * Describes all available properties and their types for the given data interface type.
	 *
	 * @param DataInterfaceClass The data interface class to get the schema for
	 * @return Schema describing all properties for the specified data interface class
	 */
	UFUNCTION(meta = (AICallable), Category = "Schema")
	static FNiagaraExt_DataInterfaceSchema GetDataInterfaceSchema(TSubclassOf<UNiagaraDataInterface> DataInterfaceClass);

	/**
	 * Returns schema for a single module input in the stack.
	 * Describes the type, metadata, and configuration options for a specific input parameter.
	 *
	 * @param InputReference Reference to the stack input to get the schema for
	 * @return Schema describing the input's type and available configuration
	 */
	UFUNCTION(meta = (AICallable), Category = "Schema")
	static FNiagaraExt_StackInputSchema GetStackInputSchema(const FNiagaraExt_StackItemReference& InputReference);

	/**
	 * Returns schema for a module and all its inputs.
	 * Call this after seeing a module in topology to understand what inputs it exposes.
	 *
	 * @param ModuleReference Reference to the module to get the schema for
	 * @return Schema describing the module's inputs and their types
	 */
	UFUNCTION(meta = (AICallable), Category = "Schema")
	static FNiagaraExt_ModuleSchema GetModuleSchema(const FNiagaraExt_StackItemReference& ModuleReference);

	/**
	 * Returns schema for a dynamic input module in the stack.
	 * Describes the inputs and configuration for a procedural value generator.
	 *
	 * @param DynamicInputReference Reference to the dynamic input to get the schema for
	 * @return Schema describing the dynamic input's parameters and their types
	 */
	UFUNCTION(meta = (AICallable), Category = "Schema")
	static FNiagaraExt_DynamicInputSchema GetDynamicInputSchema(const FNiagaraExt_StackItemReference& DynamicInputReference);

	/**
	 * Returns schema for a module asset.
	 * Standalone function that doesn't require a system context - useful for browsing available modules.
	 *
	 * @param ModuleAsset The module script asset to get the schema for
	 * @return Schema describing the module's inputs and their types
	 */
	UFUNCTION(meta = (AICallable), Category = "Schema")
	static FNiagaraExt_ModuleSchema GetModuleSchemaFromAsset(const UNiagaraScript* ModuleAsset);

	/**
	 * Returns schema for a dynamic input asset.
	 * Standalone function that doesn't require a system context - useful for browsing available dynamic inputs.
	 *
	 * @param DynamicInputAsset The dynamic input script asset to get the schema for
	 * @return Schema describing the dynamic input's parameters and their types
	 */
	UFUNCTION(meta = (AICallable), Category = "Schema")
	static FNiagaraExt_DynamicInputSchema GetDynamicInputSchemaFromAsset(const UNiagaraScript* DynamicInputAsset);

	//////////////////////////////////////////////////////////////////////////
	// Summary tier — cheap metadata-only calls, no child walk.

	/**
	 * Returns lightweight system metadata: name, user variables, and one summary entry per emitter.
	 * Use this for first contact with an unfamiliar system. For full structural detail call GetEmitterTopology per emitter.
	 *
	 * @param System The Niagara System to summarise.
	 * @return Summary containing system name, user variables, and per-emitter metadata.
	 */
	UFUNCTION(meta = (AICallable), Category = "Topology")
	static FNiagaraExt_SystemSummary GetSystemSummary(UNiagaraSystem* System);

	/**
	 * Returns lightweight emitter metadata: name, enabled state, sim target, renderer classes.
	 * Use this when you only need to check metadata without walking the full emitter structure.
	 *
	 * @param EmitterRef Reference to the emitter to summarise.
	 * @return Summary containing emitter name, enabled, sim target, de-duplicated renderer classes.
	 */
	UFUNCTION(meta = (AICallable), Category = "Topology")
	static FNiagaraExt_EmitterSummary GetEmitterSummary(const FNiagaraExt_StackItemReference& EmitterRef);

	//////////////////////////////////////////////////////////////////////////
	// Topology tier — full structural walk, no resolved input values.
	//
	// Every field on every returned struct is always populated.
	// To get resolved input values, make a parallel data call (GetEmitterInputValues etc.).
	// To get dependency rollups, call GetSystemDependencies.
	// To inspect a dynamic-input chain, call GetDynamicInputChain.

	/**
	 * Returns full emitter topology: four script stacks with all modules and inputs, renderer references.
	 * All fields always populated. The returned topology carries no input values; call GetEmitterInputValues in parallel.
	 * Note: mutation endpoints (AddEmitter, AddModule) also return topology structs — input values require a separate data call.
	 *
	 * @param EmitterRef Reference to the emitter to describe.
	 * @return Emitter topology with all four script stacks and renderer references fully walked.
	 */
	UFUNCTION(meta = (AICallable), Category = "Topology")
	static FNiagaraExt_EmitterTopology GetEmitterTopology(const FNiagaraExt_StackItemReference& EmitterRef);

	/**
	 * Returns script stack topology: all modules and their inputs in execution order.
	 * All fields always populated.
	 *
	 * @param ScriptRef Reference to the script stack to describe.
	 * @return Script stack topology with all modules and inputs fully walked.
	 */
	UFUNCTION(meta = (AICallable), Category = "Topology")
	static FNiagaraExt_ScriptStackTopology GetScriptStackTopology(const FNiagaraExt_StackItemReference& ScriptRef);

	/**
	 * Returns module topology: metadata and all inputs (name/type/visibility only, no values).
	 * All fields always populated.
	 *
	 * @param ModuleRef Reference to the module to describe.
	 * @return Module topology with all inputs in walk order.
	 */
	UFUNCTION(meta = (AICallable), Category = "Topology")
	static FNiagaraExt_ModuleTopology GetModuleTopology(const FNiagaraExt_StackItemReference& ModuleRef);

	/**
	 * Returns stack input topology: name, type, visibility, editability. No value payload.
	 * For the resolved value call GetStackInputData. For a dynamic-input chain call GetDynamicInputChain.
	 *
	 * @param StackInputRef Reference to the stack input to describe.
	 * @return Input topology with structural metadata only.
	 */
	UFUNCTION(meta = (AICallable), Category = "Topology")
	static FNiagaraExt_StackInputTopology GetStackInputTopology(const FNiagaraExt_StackItemReference& StackInputRef);

	//////////////////////////////////////////////////////////////////////////
	// Data Layer - Read and write property values

	/**
	 * Returns all user variables defined on the system.
	 * User variables are parameters exposed for external control and can be overridden per component instance.
	 *
	 * @param System The Niagara System to retrieve user variables from
	 * @return Collection of all user variables with their names, types, and default values
	 */
	UFUNCTION(meta = (AICallable), Category = "Data")
	static FNiagaraExt_UserVariables GetUserVariables(UNiagaraSystem* System);

	/**
	 * Returns system property values.
	 * Retrieves the current values of all configurable system-level properties.
	 *
	 * @param System The Niagara System to retrieve data from
	 * @return Data structure containing all system property values
	 */
	UFUNCTION(meta = (AICallable), Category = "Data")
	static FNiagaraExt_SystemData GetSystemData(UNiagaraSystem* System);

	/**
	 * Returns emitter property values as a single JSON-string blob in PropertyValues.
	 * The blob contains the full FVersionedNiagaraEmitterData (SimTarget, bLocalSpace, RandomSeed,
	 * FixedBounds, etc.) — fields use C++ PascalCase, must be parsed to read individual values.
	 *
	 * For typed access to common metadata (SimTarget, EmitterName, bEnabled, RendererClasses) prefer
	 * GetEmitterSummary — it returns those as named fields directly and avoids a JSON parse step.
	 * Use this endpoint when you need the full property set or a less-common field.
	 *
	 * @param EmitterRef Reference to the emitter to retrieve data from
	 * @return Data structure with PropertyValues set to a JSON blob of all emitter properties
	 */
	UFUNCTION(meta = (AICallable), Category = "Data")
	static FNiagaraExt_EmitterData GetEmitterData(const FNiagaraExt_StackItemReference& EmitterRef);

	/**
	 * Returns renderer property values.
	 * Retrieves the current values of all configurable renderer properties.
	 *
	 * @param RendererRef Reference to the renderer to retrieve data from
	 * @return Data structure containing all renderer property values
	 */
	UFUNCTION(meta = (AICallable), Category = "Data")
	static FNiagaraExt_RendererData GetRendererData(const FNiagaraExt_StackItemReference& RendererRef);

	/**
	 * Returns the value of a stack module input.
	 * Retrieves the current value and configuration for a specific module input parameter.
	 *
	 * @param StackInputRef Reference to the stack input to retrieve data from
	 * @return The current value and configuration of the input
	 */
	UFUNCTION(meta = (AICallable), Category = "Data")
	static FNiagaraExt_StackInputValue GetStackInputData(const FNiagaraExt_StackItemReference& StackInputRef);

	/**
	 * Returns all resolved input values for every module across all four emitter script stacks.
	 * One FNiagaraExt_ModuleInputValues entry per module, each carrying all its resolved input values.
	 * Call this in parallel with GetEmitterTopology to get both structure and values in two passes.
	 *
	 * @param EmitterRef Reference to the emitter to retrieve input values from
	 * @return Array of per-module input value bundles across all four script stacks
	 */
	UFUNCTION(meta = (AICallable), Category = "Data")
	static TArray<FNiagaraExt_ModuleInputValues> GetEmitterInputValues(const FNiagaraExt_StackItemReference& EmitterRef);

	/**
	 * Returns all resolved input values for every module in the given script stack.
	 * One FNiagaraExt_ModuleInputValues entry per module.
	 *
	 * @param ScriptRef Reference to the script stack
	 * @return Array of per-module input value bundles in execution order
	 */
	UFUNCTION(meta = (AICallable), Category = "Data")
	static TArray<FNiagaraExt_ModuleInputValues> GetScriptStackInputValues(const FNiagaraExt_StackItemReference& ScriptRef);

	/**
	 * Returns resolved input values for a single module.
	 * Use when you need values for one specific module without walking the whole emitter.
	 *
	 * @param ModuleRef Reference to the module
	 * @return Input value bundle for the module
	 */
	UFUNCTION(meta = (AICallable), Category = "Data")
	static FNiagaraExt_ModuleInputValues GetModuleInputValues(const FNiagaraExt_StackItemReference& ModuleRef);

	//////////////////////////////////////////////////////////////////////////
	// Dynamic-input chain endpoint

	/**
	 * Returns the full recursive chain for a dynamic input: topology metadata and resolved values at every level.
	 * The starting input must have value mode Dynamic; an error is surfaced otherwise.
	 * The schema expands one full level and emits a typed recursion stub at deeper levels;
	 * the wire format recurses to arbitrary depth matching the underlying chain.
	 *
	 * @param StackInputRef Reference to the dynamic input to traverse
	 * @return Recursive chain of entries, each carrying topology metadata and a resolved Value payload
	 */
	UFUNCTION(meta = (AICallable), Category = "Topology")
	static FNiagaraExt_DynamicInputChainRef GetDynamicInputChain(const FNiagaraExt_StackItemReference& StackInputRef);

	//////////////////////////////////////////////////////////////////////////
	// Dependency endpoint

	/**
	 * Returns the four Used* sets (renderers, data interfaces, modules, dynamic inputs)
	 * gathered across all emitters and system scripts.
	 * These sets are not included in topology structs; call this endpoint separately when needed.
	 *
	 * @param System The Niagara System to scan
	 * @return Dependency rollup with four distinct asset sets
	 */
	UFUNCTION(meta = (AICallable), Category = "Data")
	static FNiagaraExt_SystemDependencies GetSystemDependencies(UNiagaraSystem* System);

	//////////////////////////////////////////////////////////////////////////
	// Edit Operations - Modify system structure and properties

	/**
	 * Sets property values on a Niagara System.
	 * Applies new values to system-level properties based on the provided data structure.
	 *
	 * @param System The Niagara System to modify
	 * @param SystemData Data structure containing the property values to set
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static void SetSystemData(UNiagaraSystem* System, const FNiagaraExt_SystemData& SystemData);

	/**
	 * Sets property values on a Niagara Emitter.
	 * Applies new values to emitter-level properties based on the provided data structure.
	 *
	 * @param Emitter Reference to the emitter to modify
	 * @param EmitterData Data structure containing the property values to set
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static void SetEmitterData(const FNiagaraExt_StackItemReference& Emitter, const FNiagaraExt_EmitterData& EmitterData);

	/**
	 * Sets property values on a Niagara Renderer.
	 * Applies new values to renderer properties based on the provided data structure.
	 * Payload shape varies with the concrete renderer class; call GetRendererSchema for the
	 * renderer's class to inspect valid properties before writing.
	 *
	 * @param Renderer Reference to the renderer to modify
	 * @param RendererData Data structure containing the property values to set
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static void SetRendererData(const FNiagaraExt_StackItemReference& Renderer, const FNiagaraExt_RendererData& RendererData);

	/**
	 * Adds or updates user variables on a system.
	 * If a variable with the same name already exists, it will be replaced with the new definition.
	 *
	 * @param System The Niagara System to add variables to
	 * @param VariablesToAdd Array of user variables to add or update
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static void AddUserVariables(UNiagaraSystem* System, const TArray<FNiagaraExt_UserVariable>& VariablesToAdd);

	/**
	 * Removes user variables from a system.
	 * Deletes the specified user variables from the system's user parameter collection.
	 *
	 * @param System The Niagara System to remove variables from
	 * @param VariablesToRemove Array of variable definitions identifying which variables to remove
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static void RemoveUserVariables(UNiagaraSystem* System, const TArray<FNiagaraExt_Variable>& VariablesToRemove);

	/**
	 * Adds an emitter to a Niagara System.
	 * The new emitter will be based on the template emitter, inheriting its configuration and modules.
	 * Returns the full emitter topology (no input values — call GetEmitterInputValues for values).
	 *
	 * @param System The Niagara System to add the emitter to
	 * @param TemplateEmitter The emitter asset to use as a template for the new emitter
	 * @param EmitterName Name for the new emitter instance
	 * @return Topology of the newly added emitter with all scripts and modules walked
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static FNiagaraExt_EmitterTopology AddEmitter(UNiagaraSystem* System, UNiagaraEmitter* TemplateEmitter, FName EmitterName);

	/**
	 * Removes an emitter from a system.
	 * Deletes the specified emitter and all its associated scripts, modules, and renderers.
	 *
	 * @param EmitterToRemove Reference to the emitter to remove from the system
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static void RemoveEmitter(const FNiagaraExt_StackItemReference& EmitterToRemove);

	/**
	 * Adds a renderer to an emitter.
	 * Creates a new renderer of the specified type and adds it to the emitter's renderer list.
	 * Returns an FNiagaraExt_RendererRef with the new renderer's Index and RendererClass,
	 * usable directly with SetRendererData / GetRendererData without a follow-up topology call.
	 *
	 * @param NewRendererLocation Reference specifying which emitter to add the renderer to
	 * @param RendererClass The class of renderer to create (e.g., UNiagaraSpriteRendererProperties)
	 * @return Renderer reference (Index + RendererClass) for the newly added renderer.
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static FNiagaraExt_RendererRef AddRenderer(const FNiagaraExt_StackItemReference& NewRendererLocation, const TSubclassOf<UNiagaraRendererProperties> RendererClass);

	/**
	 * Removes a renderer from an emitter.
	 * Deletes the specified renderer from the emitter's renderer list.
	 *
	 * @param RendererToRemove Reference to the renderer to remove
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static void RemoveRenderer(const FNiagaraExt_StackItemReference& RendererToRemove);

	/**
	 * Adds a module to a script stack.
	 * The module will be inserted into the specified script's execution stack.
	 * Returns the module topology with all inputs walked (no input values — call GetModuleInputValues for values).
	 *
	 * @param ModuleLocationRef Reference specifying where to add the module
	 * @param ModuleAsset The module script asset to add to the stack
	 * @return Topology of the newly added module with all inputs populated
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static FNiagaraExt_ModuleTopology AddModule(const FNiagaraExt_StackItemReference& ModuleLocationRef, const UNiagaraScript* ModuleAsset);

	/**
	 * Removes a module from a script stack.
	 * Deletes the specified module and all its inputs from the script's execution stack.
	 *
	 * @param ModuleToRemove Reference to the module to remove from the stack
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static void RemoveModule(const FNiagaraExt_StackItemReference& ModuleToRemove);

	/**
	 * Sets whether a module is enabled.
	 * Disabled modules remain in the stack but don't execute. Current state is visible in module topology.
	 *
	 * @param ModuleRef Reference to the module to enable or disable
	 * @param bEnabled True to enable the module, false to disable it
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static void SetModuleEnabled(const FNiagaraExt_StackItemReference& ModuleRef, bool bEnabled);

	/**
	 * Adds a SetParameters module to a script stack.
	 * Unlike AddModule which requires a script asset, a SetParameters module dynamically assigns
	 * values to named parameters and generates its own internal script. Use this when you need
	 * to set one or more particle/emitter/system parameters directly in the stack.
	 *
	 * @param ModuleLocationRef Reference specifying where to add the module
	 * @param Parameters Array of parameter entries, each with a variable (name + type) and an optional default value string
	 * @return Topology of the newly added module, including all its inputs
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static FNiagaraExt_ModuleTopology AddSetParametersModule(const FNiagaraExt_StackItemReference& ModuleLocationRef, const TArray<FNiagaraExt_SetParameterEntry>& Parameters);

	/**
	 * Adds a single parameter to an existing SetParameters module.
	 * The module referenced by ModuleRef must be a SetParameters (UNiagaraNodeAssignment) module.
	 * Use bIsSetParametersModule in the module topology to confirm before calling.
	 *
	 * @param ModuleRef Reference to the existing SetParameters module
	 * @param Entry The parameter to add (variable name, type, and optional default value string)
	 * @return Updated topology of the module after the parameter is added
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static FNiagaraExt_ModuleTopology AddSetParameterEntry(const FNiagaraExt_StackItemReference& ModuleRef, const FNiagaraExt_SetParameterEntry& Entry);

	/**
	 * Removes a parameter from an existing SetParameters module by name.
	 * The module referenced by ModuleRef must be a SetParameters (UNiagaraNodeAssignment) module.
	 * Use bIsSetParametersModule in the module topology to confirm before calling.
	 *
	 * @param ModuleRef Reference to the existing SetParameters module
	 * @param ParameterName Name of the parameter to remove
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static void RemoveSetParameterEntry(const FNiagaraExt_StackItemReference& ModuleRef, FName ParameterName);

	/**
	 * Sets the value of a stack module input and returns the resulting stored value.
	 * Updates the value and configuration for a specific module input parameter.
	 *
	 * @param StackInputRef Reference to the stack input to modify
	 * @param InputData The new value and configuration to apply to the input
	 * @return The value now stored on the input after the operation
	 */
	UFUNCTION(meta = (AICallable), Category = "Edit")
	static FNiagaraExt_StackInputValue SetStackInputData(const FNiagaraExt_StackItemReference& StackInputRef, const FNiagaraExt_StackInputValue& InputData);

	//////////////////////////////////////////////////////////////////////////
	// Diagnostics Layer - Read compile state and stack issues

	/*
	 * Returns the current compile state of a Niagara System: aggregate status, per-script compile
	 * events, and summary flags. Waits for any in-flight compile to complete before collecting.
	 * @param System The Niagara System to query.
	 * @return Async result whose Value carries aggregate and per-script compile state.
	 */
	UFUNCTION(meta = (AICallable), Category = "Status")
	static UNiagaraToolset_AsyncSystemCompileState* GetSystemCompileState(UNiagaraSystem* System);

	/*
	 * Returns all stack issues (errors, warnings, info) from the Niagara module stack, including
	 * dismissed ones. Waits for any in-flight compile to complete before collecting.
	 * @param System The Niagara System to query.
	 * @return Async result whose Value carries all stack issues across the system and its emitters.
	 */
	UFUNCTION(meta = (AICallable), Category = "Status")
	static UNiagaraToolset_AsyncStackIssues* GetStackIssues(UNiagaraSystem* System);

	/*
	 * Applies a Fix-style stack issue fix identified by IssueId and FixId. Link-style fixes are
	 * rejected. The fix is undoable via the editor undo stack. Applying a fix may trigger a
	 * recompile; the result waits for that compile to complete so post-fix state is valid.
	 * @param System  The Niagara System to apply the fix to.
	 * @param IssueId IssueId from a prior FNiagaraExt_StackIssue.
	 * @param FixId   FixId from a prior FNiagaraExt_StackIssueFix.
	 * @return Async result carrying the fix outcome and a post-fix stack-issues snapshot.
	 */
	UFUNCTION(meta = (AICallable), Category = "Status")
	static UNiagaraToolset_AsyncApplyStackIssueFixResult* ApplyStackIssueFix(UNiagaraSystem* System, FString IssueId, FString FixId);

	//////////////////////////////////////////////////////////////////////////
};
