// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraToolsetsModule.h"
#include "Misc/CoreDelegates.h"

#include "NiagaraToolset_Info.h"
#include "NiagaraToolset_Component.h"
#include "NiagaraToolset_Blueprint.h"
#include "NiagaraToolset_System.h"
#include "NiagaraToolset_Assets.h"
#include "NiagaraToolsetJsonConverters.h"

#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"

#include "ToolsetRegistry/ToolsetRegistry.h"
#include "ToolsetRegistry/UToolsetRegistry.h"
#include "ToolsetRegistry/ToolsetLibrary.h"

#include "NiagaraExternalSystemEditorUtilities.h"

#define LOCTEXT_NAMESPACE "FNiagaraToolsetsModule"

void FNiagaraToolsetsModule::StartupModule()
{
 	FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddRaw(this, &FNiagaraToolsetsModule::OnAllModuleLoadingPhasesComplete);
 	FCoreDelegates::OnPreExit.AddRaw(this, &FNiagaraToolsetsModule::OnPreExit);

	RegisterToolsets();
}

void FNiagaraToolsetsModule::ShutdownModule()
{
 	FCoreDelegates::OnAllModuleLoadingPhasesComplete.RemoveAll(this);
 	FCoreDelegates::OnPreExit.RemoveAll(this);

	UnregisterToolsets();
}

void FNiagaraToolsetsModule::OnAllModuleLoadingPhasesComplete()
{
	RegisterToolsets();
}

void FNiagaraToolsetsModule::OnPreExit()
{
	UnregisterToolsets();
}

void FNiagaraToolsetsModule::RegisterToolsets()
{
	if(!bToolsetsRegistered)
	{
		auto ToolsetRegistrySubsystem = UToolsetRegistrySubsystem::Get();
		if (ToolsetRegistrySubsystem.HasError())
		{
			return;
		}

		bToolsetsRegistered = true;

		// Converters must be registered before the toolset classes: RegisterToolsetClass bakes each
		// tool's schema at registration and resolves converters then.
		InstancedValueConverter = MakeShared<FToolsetNiagaraInstancedValueConverter>();
		ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.RegisterConverter(InstancedValueConverter);

		TypeDefConverter = MakeShared<FToolsetNiagaraTypeDefinitionConverter>();
		ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.RegisterConverter(TypeDefConverter);

		UToolsetRegistry::RegisterToolsetClass(UNiagaraToolset_Info::StaticClass());
		UToolsetRegistry::RegisterToolsetClass(UNiagaraToolset_Component::StaticClass());
		UToolsetRegistry::RegisterToolsetClass(UNiagaraToolset_Blueprint::StaticClass());
		UToolsetRegistry::RegisterToolsetClass(UNiagaraToolset_System::StaticClass());
		UToolsetRegistry::RegisterToolsetClass(UNiagaraToolset_Assets::StaticClass());

		// Back Niagara's external-edit property reflection with UToolsetLibrary, which adds the
		// permission and block-list checks and container diff that the default provider omits.
		FNiagaraExternalEditPropertyProvider Provider;
		Provider.ListStructProperties = [](const UStruct* Struct, bool bUserVisibleOnly)
		{
			return UToolsetLibrary::ListStructProperties(Struct, bUserVisibleOnly);
		};
		Provider.GetObjectProperties = [](const UObject* Object, const TArray<FName>& PropertyNames)
		{
			return UToolsetLibrary::GetObjectProperties(Object, PropertyNames);
		};
		Provider.SetObjectProperties = [](UObject* Object, const FString& PropertiesJson)
		{
			return UToolsetLibrary::SetObjectProperties(Object, PropertiesJson, EBypassContainerCheck::Yes);
		};
		UNiagaraExternalEditUtilities::SetPropertyProvider(Provider);
	}
}

void FNiagaraToolsetsModule::UnregisterToolsets()
{
	if(bToolsetsRegistered)
	{
		bToolsetsRegistered = false;

		UNiagaraExternalEditUtilities::ResetPropertyProvider();

		UToolsetRegistry::UnregisterToolsetClass(UNiagaraToolset_Info::StaticClass());
		UToolsetRegistry::UnregisterToolsetClass(UNiagaraToolset_Component::StaticClass());
		UToolsetRegistry::UnregisterToolsetClass(UNiagaraToolset_Blueprint::StaticClass());
		UToolsetRegistry::UnregisterToolsetClass(UNiagaraToolset_System::StaticClass());
		UToolsetRegistry::UnregisterToolsetClass(UNiagaraToolset_Assets::StaticClass());

		auto ToolsetRegistrySubsystem = UToolsetRegistrySubsystem::Get();
		if (ToolsetRegistrySubsystem.HasError())
		{
			//Exit. We can unregister our converters if the subsystem no longer exists.
			return;
		}

		if (InstancedValueConverter)
		{
			ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.UnregisterConverter(InstancedValueConverter);
		}
		InstancedValueConverter = nullptr;

		if (TypeDefConverter)
		{
			ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.UnregisterConverter(TypeDefConverter);
		}
		TypeDefConverter = nullptr;
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FNiagaraToolsetsModule, NiagaraToolsets)
