// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

class FToolsetInstancedStructConverter;
class FToolsetNiagaraTypeDefinitionConverter;
class FToolsetNiagaraInstancedValueConverter;

class FNiagaraToolsetsModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:

	/** Called after modules are loaded so we can register with the AI Assistant. */
	void OnAllModuleLoadingPhasesComplete();

	/** Called before engine exit so we can unregister with the AI Assistant. */
	void OnPreExit();

	void RegisterToolsets();
	void UnregisterToolsets();

	bool bToolsetsRegistered = false;

	/** Json Converters */
	TSharedPtr<FToolsetNiagaraInstancedValueConverter> InstancedValueConverter;
	TSharedPtr<FToolsetNiagaraTypeDefinitionConverter> TypeDefConverter;
};
