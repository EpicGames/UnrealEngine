// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraToolsetsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraToolsetsSettings)

FName UNiagaraToolsetsSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

FText UNiagaraToolsetsSettings::GetSectionText() const
{
	return NSLOCTEXT("NiagaraToolsetsPlugin", "NiagaraToolsetsSection", "Niagara Toolsets");
}

