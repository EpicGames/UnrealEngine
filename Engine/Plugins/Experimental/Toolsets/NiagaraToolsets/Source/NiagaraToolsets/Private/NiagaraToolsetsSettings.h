// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "NiagaraToolsetsCommon.h"
#include "NiagaraToolset_Assets.h"

#include "NiagaraToolsetsSettings.generated.h"

UCLASS(config = NiagaraToolsets, defaultconfig, meta=(DisplayName="Niagara Toolsets"), MinimalAPI)
class UNiagaraToolsetsSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	// Begin UDeveloperSettings Interface
	virtual FName GetCategoryName() const override;
	virtual FText GetSectionText() const override;
	// END UDeveloperSettings Interface

	/** Information given to the AI about where it should look for assets when working with Niagara. */
	UPROPERTY(config, EditAnywhere, Category="Niagara Toolsets")
	TArray<FNiagaraToolsetAssetDiscoveryGroup> AssetDiscoveryGroups;
};
